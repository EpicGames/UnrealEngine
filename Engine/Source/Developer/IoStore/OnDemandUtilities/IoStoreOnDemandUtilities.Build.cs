// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;

public class IoStoreOnDemandUtilities : ModuleRules
{
	public IoStoreOnDemandUtilities(ReadOnlyTargetRules Target) : base(Target)
	{
		PublicDependencyModuleNames.Add("Core");
		PublicDependencyModuleNames.Add("TraceLog");
		PrivateDependencyModuleNames.AddRange(
			new string[] {
				"CoreUObject",
				"IoStoreOnDemand",
				"IoStoreOnDemandCore",
				"IoStoreHttpClient",
				"Json"
			}
		);

		CppCompileWarningSettings.UnsafeTypeCastWarningLevel = WarningLevel.Error; 

		if (Target.Platform.IsInGroup(UnrealPlatformGroup.Desktop) &&
			(Target.Type == TargetType.Editor || Target.Type == TargetType.Program))
		{
			PrivateDependencyModuleNames.AddRange(new string[] { "S3Client", "RSA" });
		}
	}
}
