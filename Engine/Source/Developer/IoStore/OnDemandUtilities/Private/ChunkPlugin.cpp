// Copyright Epic Games, Inc. All Rights Reserved.

#include "Command.h"
#include "Common.h"
#include "Upload.h"

#include "Algo/Find.h"
#include "Containers/StringView.h"
#include "HAL/FileManager.h"
#include "IO/IoChunkEncoding.h"
#include "IO/IoContainerHeader.h"
#include "IO/IoStatus.h"
#include "IO/IoStore.h"
#include "IO/IoStoreOnDemand.h"
#include "IO/Serialization/OnDemandContainerToc.h"
#include "Misc/DateTime.h"
#include "Misc/FileHelper.h"
#include "Misc/KeyChainUtilities.h"
#include "Misc/PathViews.h"
#include "Misc/Paths.h"
#include "Misc/StringBuilder.h"
#include "S3/S3Client.h"
#include "Serialization/JsonReader.h"
#include "Serialization/JsonSerializer.h"
#include "Serialization/JsonSerializerMacros.h"
#include "Serialization/JsonWriter.h"
#include "Serialization/LargeMemoryWriter.h"
#include "Serialization/MemoryReader.h"
#include "UploadQueue.h"

namespace UE::IoStore::Tool
{

using FOnDemandTocWriter			= UE::IoStore::Serialization::FOnDemandTocWriter;
using EOnDemandContainerEntryFlags	= UE::IoStore::Serialization::EOnDemandContainerEntryFlags;

struct FChunkPluginStats : FJsonSerializable
{
	int64 IADTocSize = 0;
	int64 IADChunksSize = 0;

	int64 IASTocSize = 0;
	int64 IASChunksSize = 0;

	BEGIN_JSON_SERIALIZER
		JSON_SERIALIZE("IADTocSize",	IADTocSize);
		JSON_SERIALIZE("IADChunksSize",	IADChunksSize);

		JSON_SERIALIZE("IASTocSize",	IASTocSize);
		JSON_SERIALIZE("IASChunksSize",	IASChunksSize);
	END_JSON_SERIALIZER
};

struct FS3Params
{
public:
	FString S3ServiceUrl;
	FString S3Bucket;
	FString S3BucketPrefix;
	FString S3Region;
	FString S3AccessKey;
	FString S3SecretKey;
	FString S3SessionToken;
	int32 S3MaxConcurrentUploads;

	FS3Params(const FContext& Context)
	{
		S3ServiceUrl = FString(Context.Get<FStringView>(TEXT("-ServiceUrl"), FString()));
		S3Bucket = FString(Context.Get<FStringView>(TEXT("-Bucket"), FString()));
		S3Region = FString(Context.Get<FStringView>(TEXT("-Region"), FString()));
		S3AccessKey = FString(Context.Get<FStringView>(TEXT("-AccessKey"), FString()));
		S3SecretKey = FString(Context.Get<FStringView>(TEXT("-SecretKey"), FString()));
		S3SessionToken = FString(Context.Get<FStringView>(TEXT("-SessionToken"), FString()));


		S3BucketPrefix = FString(Context.Get<FStringView>(TEXT("-BucketPrefix"), FString()));
		S3MaxConcurrentUploads = Context.Get<int32>(TEXT("-MaxConcurrentUploads"), 10);
	}

	bool IsValid()
	{
		return !S3AccessKey.IsEmpty() && !S3SecretKey.IsEmpty() && !S3SessionToken.IsEmpty();
	}
};


////////////////////////////////////////////////////////////////////////////////
struct FChunkPluginSettings
{
	int32 LoadFromFile(const FString& SettingsFile)
	{
		if (!SettingsFile.IsEmpty())
		{
			UE_LOGF(LogIoStoreOnDemand, Display, "Loading settings file '%ls'...", *SettingsFile);

			TUniquePtr<FArchive> Ar(IFileManager::Get().CreateFileReader(*SettingsFile));
			if (!Ar || Ar->IsError())
			{
				UE_LOGF(LogIoStoreOnDemand, Error, "Failed to open settings file '%ls'", *SettingsFile);
				return -1;
			}

			TSharedRef<TJsonReader<UTF8CHAR>> JsonReader = TJsonReaderFactory<UTF8CHAR>::Create(Ar.Get());

			TSharedPtr<FJsonValue> JsonSettings;
			if (!FJsonSerializer::Deserialize(*JsonReader, JsonSettings))
			{
				UE_LOGF(LogIoStoreOnDemand, Error, "Failed to read settings file '%ls'", *SettingsFile);
				return -1;
			}

			TSharedPtr<FJsonObject> JsonSettingsObject = JsonSettings->AsObject();
			if (!JsonSettingsObject)
			{
				UE_LOGF(LogIoStoreOnDemand, Error, "Bad settings file '%ls'", *SettingsFile);
				return -1;
			}

			if (TSharedPtr<FJsonValue> PackageSetsValue = JsonSettingsObject->TryGetField(TEXT("PackageSets")))
			{
				TSharedPtr<FJsonObject> PackageSetsObject = PackageSetsValue->AsObject();
				if (!PackageSetsObject)
				{
					UE_LOGF(LogIoStoreOnDemand, Error, "Bad settings file '%ls'", *SettingsFile);
					return -1;
				}

				for (const TPair<FJsonObject::FStringType, TSharedPtr<FJsonValue>>& Pair : PackageSetsObject->Values)
				{
					TArray<FString> Packages;
					if (!PackageSetsObject->TryGetStringArrayField(Pair.Key, Packages))
					{
						UE_LOGF(LogIoStoreOnDemand, Error, "Bad settings file '%ls'", *SettingsFile);
						return -1;
					}

					UE_LOGF(LogIoStoreOnDemand, Display, "Found Package Set '%ls' with %d entries", *Pair.Key, Packages.Num());
					for (const FString& Package : Packages)
					{
						UE_LOGF(LogIoStoreOnDemand, Log, "->\t'%ls'", *Package);
					}

					PackageSets.Add(FString(Pair.Key), MoveTemp(Packages));
				}
			}
		}

		return 0;
	}

	TMap<FString, TArray<FString>> PackageSets;
};

class FChunkWriterInterface
{
public:
	virtual ~FChunkWriterInterface() {}
	virtual FIoStatus WriteChunk(const FString& RelativeDir, FIoBuffer Chunk, const FIoHash& Hash) = 0;

	virtual bool Flush() = 0;
};



class FS3ChunkWriter : public FChunkWriterInterface
{
public:
	FS3ChunkWriter(const FS3Params& S3Params) : 
		BucketPrefix(S3Params.S3BucketPrefix),
		Client(FS3ClientConfig({ S3Params.S3Region, S3Params.S3ServiceUrl }), FS3ClientCredentials(S3Params.S3AccessKey, S3Params.S3SecretKey, S3Params.S3SessionToken)),
		UploadQueue(Client, S3Params.S3Bucket, S3Params.S3MaxConcurrentUploads)
	{
	}

	virtual ~FS3ChunkWriter()
	{
		// this should have already been done but do it now just in case
		UploadQueue.Flush();
	}


	virtual FIoStatus WriteChunk(const FString& RelativeDirectory, FIoBuffer Chunk, const FIoHash& Hash) override
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(FS3ChunkWriter::WriteChunk);

		const FString HashString = LexToString(Hash);

		TStringBuilder<256> Key;
		Key << BucketPrefix << TEXT("/")
			<< RelativeDirectory
			<< TEXT("/") << HashString.Left(2)
			<< TEXT("/") << HashString
			<< UE::IoStore::Serialization::FOnDemandFileExt::Partition;

		if (UploadQueue.Enqueue(Key, Chunk) == false)
		{
			return FIoStatus(EIoErrorCode::WriteError, TEXT("Failed to upload chunk"));
		}

		return FIoStatus(EIoErrorCode::Ok);
	}

	virtual bool Flush() override
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(FS3ChunkWriter::Flush);

		if (!UploadQueue.Flush())
		{
			return false;
		}
		return true;
	}


private:
	FString BucketPrefix;
	UE::FS3Client Client;
	FUploadQueue UploadQueue;
};


class FDiskChunkWriter : public FChunkWriterInterface
{
public:
	FDiskChunkWriter(const FString& InOutputFolder)
		: OutputFolder(InOutputFolder)
	{
	}

	virtual ~FDiskChunkWriter() = default;

	virtual  FIoStatus WriteChunk(const FString& Directory, FIoBuffer Chunk, const FIoHash& Hash) override 
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(FDiskChunkWriter::WriteChunk);

		IFileManager& FileMgr = IFileManager::Get();
		const FString HashString = LexToString(Hash);

		TStringBuilder<256> Sb;
		Sb << OutputFolder << TEXT("/") << Directory << TEXT("/") << HashString.Left(2);

		bool bTree = true;
		if (FileMgr.MakeDirectory(Sb.ToString(), bTree) == false)
		{
			return FIoStatusBuilder(EIoErrorCode::WriteError)
				<< TEXT("Failed to create directory '")
				<< FString(Sb.ToString())
				<< TEXT("'");
		}

		Sb << TEXT("/") << HashString << UE::IoStore::Serialization::FOnDemandFileExt::Partition;

		TUniquePtr<FArchive> Ar(IFileManager::Get().CreateFileWriter(Sb.ToString()));

		if (!Ar.IsValid())
		{
			return FIoStatusBuilder(EIoErrorCode::WriteError) << TEXT("Failed to open handle to write file '") << FString(Sb.ToString()) << TEXT("'");
		}

		Ar->Serialize((void*)Chunk.GetView().GetData(), Chunk.GetView().GetSize());

		if (!Ar->Close())
		{
			return FIoStatusBuilder(EIoErrorCode::WriteError) << TEXT("Failed to write to file '") << FString(Sb.ToString()) << TEXT("'");
		}

		UE_LOGF(LogIoStoreOnDemand, Log, "Writing file '%ls' (%.2lf KiB)", Sb.ToString(), double(Chunk.GetView().GetSize()) / 1024.0);
		return EIoErrorCode::Ok;
	}

	bool Flush() override
	{
		return true;
	}

private:
	FString OutputFolder;
};

struct FChunkParams
{
	FChunkPluginSettings Settings;
	TMap<FGuid, FAES::FAESKey> EncryptionKeys;

	TUniquePtr<FChunkWriterInterface> ChunkWriter;
	FString ChunksRelativeFolder;

	int32 MaxPartitionSize = 0;
	bool bValidateChunks = false;
};

struct FChunkOutput
{
	FChunkOutput(const FString& HostGroupName, const FString& Platform, const FString& BuildVersion)
	{
		const TCHAR* ChunksDirectory = TEXT(""); // TODO: Should we be setting this to something?

		InstallOnDemandToc.SetMetadata(BuildVersion, Platform);
		InstallOnDemandToc.SetChunksDirectory(ChunksDirectory);
		InstallOnDemandToc.SetHostGroup(HostGroupName);

		StreamOnDemandToc.SetMetadata(BuildVersion, Platform);
		StreamOnDemandToc.SetChunksDirectory(ChunksDirectory);
		StreamOnDemandToc.SetHostGroup(HostGroupName);
	}

	FOnDemandTocWriter& GetToc(EOnDemandContainerEntryFlags Flags)
	{
		if (EnumHasAnyFlags(Flags, EOnDemandContainerEntryFlags::InstallOnDemand))
		{
			return InstallOnDemandToc;
		}
		else
		{
			return StreamOnDemandToc;
		}
	}

	FOnDemandTocWriter InstallOnDemandToc;
	FOnDemandTocWriter StreamOnDemandToc;

	/* The paths of the container files (.ucas, .utoc etc) that were chunked. */
	TArray<FString> ContainerFilePaths;
	/* Statistics about the chunking process. */
	FChunkPluginStats Stats;
};

static int32 ChunkContainer(const FString& ContainerUTocPath, EOnDemandContainerEntryFlags ContainerFlags, const FChunkParams& Input, FChunkOutput& Output)
{
	using namespace UE::IoStore::Tool::Upload;

	TRACE_CPUPROFILER_EVENT_SCOPE(ChunkPluginCommand::ChunkContainer);

	check(Input.ChunkWriter.IsValid());

	TConstArrayView<FString> FileNames(&ContainerUTocPath, 1);
	TPagedArray<FContainerData> Containers;
	const bool bIgnoreContainerFlags = true;
	if (FIoStatus Status = LoadContainers(
		FileNames,
		Input.EncryptionKeys,
		Containers,
		bIgnoreContainerFlags); !Status.IsOk())
	{
		UE_LOGF(LogIoStoreOnDemand, Error, "Failed to open container '%ls' for reading due to: %ls",
			*ContainerUTocPath, *Status.ToString());
		return -1;
	}

	if (Containers.IsEmpty())
	{
		UE_LOGF(LogIoStoreOnDemand, Error, "Failed to load container '%ls'", *ContainerUTocPath);
		return -1;
	}

	const FTagSets& TagSets = Input.Settings.PackageSets;
	if (EnumHasAnyFlags(ContainerFlags, EOnDemandContainerEntryFlags::InstallOnDemand) && !TagSets.IsEmpty())
	{
		// Can only create tag sets for containers with package information
		if (Containers[0].Header.GetSize() > 0)
		{
			if (FIoStatus Status = CreateTagSets(TagSets, Containers[0]); !Status.IsOk())
			{
				UE_LOGF(LogIoStoreOnDemand, Error, "Failed to create tag set(s) for container '%ls' due to: %ls",
						*ContainerUTocPath, *Status.ToString());
				return -1;
			}
		}
	}

	UE_LOGF(LogIoStoreOnDemand, Display, "Processing container '%ls'", *ContainerUTocPath);

	FOnDemandTocWriter& TocWriter = Output.GetToc(ContainerFlags);

	auto WriteChunk = [&Input](FIoBuffer Chunk, const FIoHash& Hash) -> FIoStatus
	{
		return Input.ChunkWriter->WriteChunk(Input.ChunksRelativeFolder, Chunk, Hash);
	};

	FChunkValidationSettings ValidationSettings
	{
		.bValidate = Input.bValidateChunks,
		.MaxRawChunkSize = 32 << 20
	};

	TSet<FIoHash> ExistingChunks;
	if (FIoStatus Status = UploadContainer(
		Containers[0],
		Input.EncryptionKeys,
		Input.MaxPartitionSize,
		ContainerFlags,
		ExistingChunks,
		MoveTemp(WriteChunk),
		TocWriter,
		ValidationSettings); !Status.IsOk())
	{
		UE_LOGF(LogIoStoreOnDemand, Error, "Failed to process container '%ls' due to: %ls",
			*ContainerUTocPath, *Status.ToString());
		return -1;
	}

	Output.ContainerFilePaths.Add(ContainerUTocPath);						// .utoc
	Containers[0].Reader->GetContainerFilePaths(Output.ContainerFilePaths);	// .ucas

	return 0;
}

int32 WriteOnDemandToc(EOnDemandContainerEntryFlags TocFlags, FOnDemandTocWriter& OnDemandToc, const FString& ContainerName, const FString& OutputFolder, FChunkPluginStats& OutStats)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(ChunkPluginCommand::WriteOnDemandToc);

	IFileManager& FileMgr = IFileManager::Get();

	FString Filename = FPathViews::SetExtension(ContainerName, UE::IoStore::Serialization::FOnDemandFileExt::Toc);
	Filename.ToLowerInline();

	const FString TocPath = OutputFolder / Filename;

	TIoStatusOr<uint64> Status = OnDemandToc.Write(TocPath);
	if (!Status.IsOk())
	{
		UE_LOGF(LogIoStoreOnDemand, Error, "Failed to serialize on-demand container TOC '%ls', reason: %ls", *TocPath, *Status.Status().ToString());
		return -1;
	}

	const uint64 TocSize = Status.ConsumeValueOrDie();

	if (EnumHasAnyFlags(TocFlags, EOnDemandContainerEntryFlags::InstallOnDemand))
	{
		OutStats.IADTocSize += TocSize;
	}
	else
	{
		OutStats.IASTocSize += TocSize;
	}

	UE_LOGF(LogIoStoreOnDemand, Display, "Saved on-demand container TOC '%ls' (%.2lf KiB)", *TocPath, static_cast<double>(TocSize) / 1024.0);
	
	return 0;
}

////////////////////////////////////////////////////////////////////////////////
static int32 ChunkPluginCommandEntry(const FContext& Context)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(ChunkPluginCommand);
	//while (!FPlatformMisc::IsDebuggerPresent());
	//UE_DEBUG_BREAK();

	const FString Platform				= FString(Context.Get<FStringView>(TEXT("-Platform"), FString()));
	const FString BuildVersion			= FString(Context.Get<FStringView>(TEXT("-BuildVersion"), FString()));
	const FString OnDemandTocName		= FString(Context.Get<FStringView>(TEXT("-OnDemandTocName"), FString()));
	const FString InputFolder			= FString(Context.Get<FStringView>(TEXT("-InputFolder"), FString()));
	const FString InOutputFolder		= FString(Context.Get<FStringView>(TEXT("-OutputFolder"), FString()));
	const FString IntermediateFolder	= FString(Context.Get<FStringView>(TEXT("-IntermediateFolder"), FString()));
	FString SettingsFile				= FString(Context.Get<FStringView>(TEXT("-SettingsFile"), FString()));
	FString OutputStatsJson				= FString(Context.Get<FStringView>(TEXT("-OutputStatsJson"), FString()));
	const FString HostGroupName			= FString(Context.Get<FStringView>(TEXT("-HostGroupName"), FString()));
	int32 PartitionSize					= Context.Get<int32>(TEXT("-MaxPartitionSize"), Upload::FUploadSettings::DefaultPartitionSize);

	const bool bSupportInstalledContainers = !Context.Get<bool>(TEXT("-SkipInstallOnDemand"), false);
	const bool bSupportStreamingContainers = !Context.Get<bool>(TEXT("-SkipStreamOnDemand"), false);

	FS3Params S3Params(Context);

	const bool bIncludeSigPak			= Context.Get<bool>(TEXT("-IncludeSigPak"), false);
	const bool bDeleteContainerFiles	= !Context.Get<bool>(TEXT("-KeepContainerFiles"), false);
	const bool bValidateChunks			= Context.Get<bool>(TEXT("-ValidateChunks"), false);

	FString OutputFolder				= InOutputFolder;
	FString ContainerFolder				= InputFolder;
	FString IoStoreRelativeFolder		= "iostore";
	FString ChunksRelativeFolder		= IoStoreRelativeFolder / TEXT("chunks");

	FPaths::NormalizeDirectoryName(ContainerFolder);
	FPaths::NormalizeDirectoryName(OutputFolder);
	FPaths::NormalizeFilename(SettingsFile);
	FPaths::NormalizeFilename(OutputStatsJson);

	UE_LOGF(LogIoStoreOnDemand, Display, "I/O store chunk plugin:");
	UE_LOGF(LogIoStoreOnDemand, Display, "----------------------------------------");
	UE_LOGF(LogIoStoreOnDemand, Display, "\tBuildVersion: %ls", *BuildVersion);
	UE_LOGF(LogIoStoreOnDemand, Display, "\tPlatform: %ls", *Platform);
	UE_LOGF(LogIoStoreOnDemand, Display, "\tOnDemandTocName: %ls", *OnDemandTocName);
	UE_LOGF(LogIoStoreOnDemand, Display, "\tInputFolder: %ls", *InputFolder);
	UE_LOGF(LogIoStoreOnDemand, Display, "\tOutputFolder: %ls", *OutputFolder);
	UE_LOGF(LogIoStoreOnDemand, Display, "\tIntermediateFolder: %ls", *IntermediateFolder);
	UE_LOGF(LogIoStoreOnDemand, Display, "\tSkipInstallOnDemand: %ls", !bSupportInstalledContainers ? TEXT("true") : TEXT("false"));
	UE_LOGF(LogIoStoreOnDemand, Display, "\tSkipStreamOnDemand: %ls", !bSupportStreamingContainers ? TEXT("true") : TEXT("false"));
	UE_LOGF(LogIoStoreOnDemand, Display, "\tSettingsFile: %ls", *SettingsFile);
	UE_LOGF(LogIoStoreOnDemand, Display, "\tOutputStatsJson: %ls", *OutputStatsJson);
	UE_LOGF(LogIoStoreOnDemand, Display, "\tIncludeSigPak: %ls", bIncludeSigPak ? TEXT("true") : TEXT("false"));
	UE_LOGF(LogIoStoreOnDemand, Display, "\tDeleteContainerFiles: %ls", bDeleteContainerFiles ? TEXT("true") : TEXT("false"));
	UE_LOGF(LogIoStoreOnDemand, Display, "\tValidateChunks: %ls", bValidateChunks ? TEXT("true") : TEXT("false"));

	FChunkParams Input;
	Input.ChunksRelativeFolder = ChunksRelativeFolder;
	Input.bValidateChunks = bValidateChunks;
	Input.MaxPartitionSize = FMath::Clamp(PartitionSize, 0, Upload::FUploadSettings::MaxPartitionSize);

	IFileManager& FileMgr = IFileManager::Get();

	if (S3Params.IsValid())
	{
		Input.ChunkWriter = MakeUnique<FS3ChunkWriter>(S3Params);
	}
	else
	{
		Input.ChunkWriter = MakeUnique<FDiskChunkWriter>(OutputFolder);
	}

	if (int32 Result = Input.Settings.LoadFromFile(SettingsFile); Result < 0)
	{
		return Result;
	}

	{
		FKeyChain KeyChain = Common::LoadCryptoKeys(Context);
		for (const TPair<FGuid, FNamedAESKey>& KeyPair : KeyChain.GetEncryptionKeys())
		{
			Input.EncryptionKeys.Add(KeyPair.Key, KeyPair.Value.Key);
		}
	}

	if (FileMgr.DirectoryExists(*ContainerFolder) == false)
	{
		UE_LOGF(LogIoStoreOnDemand, Error, "Directory '%ls' does not exist", *ContainerFolder);
		return -1;
	}

	TArray<FString> ContainerFilenames;
	FileMgr.FindFiles(ContainerFilenames, *ContainerFolder, TEXT("*.utoc"));
	UE_LOGF(LogIoStoreOnDemand, Display, "Found %d container files(s)", ContainerFilenames.Num());

	if (!bSupportInstalledContainers && !bSupportStreamingContainers)
	{
		UE_LOGF(LogIoStoreOnDemand, Error, "Neither Installed or Streaming containers were requested");
		return -1;
	}

	FChunkOutput Output(HostGroupName, Platform, BuildVersion);

	for (const FString& Filename : ContainerFilenames)
	{
		const FString FullPath = ContainerFolder / Filename;

		// TODO: At the moment the only way we can identify an OnDemand container is by the name. Might be better to pass this info into the tool
		const bool bIsStreamOnDemandContainer = FullPath.Contains(TEXT("OnDemand"));

		EOnDemandContainerEntryFlags ContainerFlags = EOnDemandContainerEntryFlags::None;

		if (bSupportStreamingContainers && bIsStreamOnDemandContainer)
		{
			ContainerFlags = EOnDemandContainerEntryFlags::StreamOnDemand;
		}
		else if (bSupportInstalledContainers && !bIsStreamOnDemandContainer)
		{
			ContainerFlags = EOnDemandContainerEntryFlags::InstallOnDemand;
		}
		else
		{
			UE_LOGF(LogIoStoreOnDemand, Display, "Skipping container '%ls'", *FullPath);
			continue;
		}

		if (int32 Result = ChunkContainer(FullPath, ContainerFlags, Input, Output); Result < 0)
		{
			return Result;
		}
	}

	if (bDeleteContainerFiles)
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(ChunkPluginCommand::DeleteContainerFiles);

		UE_LOGF(LogIoStoreOnDemand, Display, "Deleting chunked  containers...");

		for (const FString& Path : Output.ContainerFilePaths)
		{
			if (FileMgr.FileExists(*Path))
			{
				UE_LOGF(LogIoStoreOnDemand, Display, "Deleting '%ls'", *Path);
				if (!FileMgr.Delete(*Path, /*RequireExists*/true))
				{
					UE_LOGF(LogIoStoreOnDemand, Error, "Failed to delete '%ls'", *Path);
				}
			}
		}
	}

	// Write additional file(s)
	if (bIncludeSigPak)
	{
		UE_LOGF(LogIoStoreOnDemand, Warning, "Adding additional file(s) to the on-demand container TOC is no longer supported");
	}

	// todo, we actually want the .uondemandtoc files in the base directory
	// const FString TocPath = OutputFolder / Filename; // like this yo
	const FString OnDemandTocOutputFolder = OutputFolder / IoStoreRelativeFolder;

	if (bSupportInstalledContainers && !Output.InstallOnDemandToc.IsEmpty())
	{
		if (int32 Result = WriteOnDemandToc(EOnDemandContainerEntryFlags::InstallOnDemand, Output.InstallOnDemandToc, OnDemandTocName, OnDemandTocOutputFolder, Output.Stats); Result < 0)
		{
			return Result;
		}
	}

	if (bSupportStreamingContainers && !Output.StreamOnDemandToc.IsEmpty())
	{
		// TODO: Improve how we choose the name here.
		if (int32 Result = WriteOnDemandToc(EOnDemandContainerEntryFlags::StreamOnDemand, Output.StreamOnDemandToc, OnDemandTocName + TEXT("ondemand"), OnDemandTocOutputFolder, Output.Stats); Result < 0)
		{
			return Result;
		}
	}

	// Write dummy containers if necessary
	if (bDeleteContainerFiles)
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(ChunkPluginCommand::WriteDummyContainerFiles);

		UE_LOGF(LogIoStoreOnDemand, Display, "Replacing .utoc for chunked  containers...");

		for (const FString& SourcePath : Output.ContainerFilePaths)
		{
			if (!SourcePath.EndsWith(TEXT(".utoc")))
			{
				continue;
			}

			FNameBuilder NameBuilder;
			NameBuilder << BuildVersion << FPathViews::GetBaseFilename(SourcePath) << TEXTVIEW("dummy");

			FIoContainerSettings ContainerSettings;
			ContainerSettings.ContainerId = FIoContainerId::FromName(FName(NameBuilder));

			const FString FullPath = ContainerFolder / FPaths::GetCleanFilename(SourcePath);

			FIoStoreTocResource Toc;

			TIoStatusOr<uint64> Status = FIoStoreTocResource::Write(*FullPath, Toc, 0, 0, ContainerSettings);
			if (!Status.IsOk())
			{
				UE_LOGF(LogIoStoreOnDemand, Error, "Failed to write dummy container '%ls' (%ls)", *FullPath, *Status.Status().ToString());
				return -1;
			}

			UE_LOGF(LogIoStoreOnDemand, Display, "Wrote dummy file '%ls' (%.2lf KiB)", *FullPath, double(Status.ValueOrDie()) / 1024);
		}
	}

	if (!Input.ChunkWriter->Flush())
	{
		UE_LOGF(LogIoStoreOnDemand, Error, "Writer error: Failed to upload chunk(s)");
		return -1; 
	}

	if (!OutputStatsJson.IsEmpty())
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(ChunkPluginCommand::WritStatisticsFile);

		TUniquePtr<FArchive> FileWriter(FileMgr.CreateFileWriter(*OutputStatsJson));
		if (!FileWriter)
		{
			UE_LOGF(LogIoStoreOnDemand, Display, "Failed writing stats file '%ls'", *OutputStatsJson);
			return -1;
		}

		{
			TSharedRef<TJsonWriter<UTF8CHAR>> JsonWriter = TJsonWriterFactory<UTF8CHAR>::Create(FileWriter.Get());
			Output.Stats.ToJson(JsonWriter, false);

			if (!JsonWriter->Close()) // Verify that the json doc was formatted correctly.
			{
				UE_LOGF(LogIoStoreOnDemand, Display, "Failed writing stats file '%ls'", *OutputStatsJson);
				return -1;
			}
		}

		if (!FileWriter->Close())
		{
			UE_LOGF(LogIoStoreOnDemand, Display, "Failed writing stats file '%ls'", *OutputStatsJson);
			return -1;
		}

		UE_LOGF(LogIoStoreOnDemand, Display, "Stats file written to '%ls'", *OutputStatsJson);
	}

	return 0;
}

////////////////////////////////////////////////////////////////////////////////
extern FArgumentSet S3Arguments;

static FCommand ChunkPluginCommand(
	ChunkPluginCommandEntry,
	TEXT("ChunkPlugin"),
	TEXT(""),
	{
		TArgument<FStringView>(TEXT("-Platform"),			TEXT("Platform name.")),
		TArgument<FStringView>(TEXT("-BuildVersion"),		TEXT("Build version")),
		TArgument<FStringView>(TEXT("-OnDemandTocName"),	TEXT("On Demand TOC Name")),
		TArgument<FStringView>(TEXT("-InputFolder"),		TEXT("Input folder to plugin information.")),
		TArgument<FStringView>(TEXT("-OutputFolder"),		TEXT("Ouptut folder.")),
		TArgument<FStringView>(TEXT("-IntermediateFolder"),	TEXT("Intermediate folder.")),
		TArgument<FStringView>(TEXT("-SettingsFile"),		TEXT("Optional settings file.")),
		TArgument<FStringView>(TEXT("-OutputStatsJson"),	TEXT("Path to write a json file with statistics.")),
		TArgument<FStringView>(TEXT("-HostGroupName"),		TEXT("Host group name or URL")),
		TArgument<FStringView>(TEXT("-CryptoKeys"),			TEXT("")),
		TArgument<bool>(TEXT("-IncludeSigPak"),				TEXT("Include .sig and .pak file in the uondemandtoc")),
		TArgument<bool>(TEXT("-KeepContainerFiles"),		TEXT("Should we keep the container files after processing them.")),
		TArgument<bool>(TEXT("-SkipStreamOnDemand"),		TEXT("Do not chunk any stream on-demand containers")),
		TArgument<bool>(TEXT("-SkipInstallOnDemand"),		TEXT("Do not chunk any install on-demand containers")),
		TArgument<FStringView>(TEXT("-BucketPrefix"),		TEXT("Path to prefix to bucket objects")),
		TArgument<int32>(TEXT("-MaxConcurrentUploads"),		TEXT("Number of simultaneous uploads")),
		TArgument<int32>(TEXT("-MaxPartitionSize"),			TEXT("Max partition size in bytes.")),
		TArgument<bool>(TEXT("-ValidateChunks"),			TEXT("Runs additional validation on chunks after they are read from the container and before they are written out")),
		S3Arguments,
	}
);

} // namespace UE::IoStore::Tool
