// Copyright Epic Games, Inc. All Rights Reserved.

#include "Command.h"

#include <Containers/UnrealString.h>
#include <CoreGlobals.h>
#include <Misc/OutputDeviceRedirector.h>
#include <Misc/ScopeExit.h>

#if PLATFORM_WINDOWS
#	include "Windows/AllowWindowsPlatformTypes.h"
#		include <errhandlingapi.h>
#	include "Windows/HideWindowsPlatformTypes.h"
#endif

namespace UE::IoStore::Tool {

////////////////////////////////////////////////////////////////////////////////
namespace Error
{

struct FBase
{
					FBase(const TCHAR* Ever, FStringView Not) : What(Ever), Why(Not) {}
	const TCHAR*	What;
	FString			Why;
};

#define MAKE_IASTOOL_ERROR(Name) \
	struct Name	: public FBase { \
		Name(FStringView Not) : FBase(TEXT(#Name), Not) {} \
	};
	MAKE_IASTOOL_ERROR(FMissingValue);
	MAKE_IASTOOL_ERROR(FCommandNotFound);
	MAKE_IASTOOL_ERROR(FBoolPositional);
	MAKE_IASTOOL_ERROR(FUnexpectedValue);
	MAKE_IASTOOL_ERROR(FUnknownKey);
	MAKE_IASTOOL_ERROR(FCommandAbort);
#undef MAKE_IASTOOL_ERROR

}

////////////////////////////////////////////////////////////////////////////////
template <typename ErrorType, typename... ArgTypes>
requires std::is_base_of<Error::FBase, ErrorType>::value
[[noreturn]] void FatalError(ArgTypes&&... Args)
{
	throw ErrorType(Forward<ArgTypes>(Args)...);
}



////////////////////////////////////////////////////////////////////////////////
static void WriteLine(const TCHAR* Format, ...)
{
	va_list VaList;
	va_start(VaList, Format);
	ON_SCOPE_EXIT { va_end(VaList); };

	TCHAR Buffer[256];
	FCString::GetVarArgs(Buffer, UE_ARRAY_COUNT(Buffer), Format, VaList);

	static FName Category = FName("IasTool");
	GLog->Serialize(Buffer, ELogVerbosity::Type::Display, Category);
}

////////////////////////////////////////////////////////////////////////////////
static bool IsAnAstleySingleFrom91(const TCHAR* Cassette)
{
	if (Cassette[0] != '-' && Cassette[0] != '/')
	{
		return false;
	}

	FStringView Cd = Cassette + 1;
	for (const TCHAR* TwelveInch : { TEXT("help"), TEXT("h"), TEXT("?"), TEXT("-help") }) // '/-help' :)
	{
		if (Cd.Equals(TwelveInch, ESearchCase::IgnoreCase))
		{
			return true;
		}
	}

	return false;
}



////////////////////////////////////////////////////////////////////////////////
using ParseFunc = void (void*, FStringView);

////////////////////////////////////////////////////////////////////////////////
template <typename Type>
struct TArgumentParse
{
	static ParseFunc* GetParser()
	{
		return [] (void* Out, FStringView Input) {
			auto* Inner = (Type*)Out;
			LexFromString(*Inner, Input);
		};
	}
};

////////////////////////////////////////////////////////////////////////////////
template <>
struct TArgumentParse<bool>
{
	static ParseFunc* GetParser() { return nullptr; }
};

////////////////////////////////////////////////////////////////////////////////
template <>
struct TArgumentParse<FStringView>
{
	static ParseFunc* GetParser()
	{
		return [] (void* Out, FStringView Input) {
			auto* Inner = (FStringView*)Out;
			new (Inner) FStringView(Input);
		};
	}
};

////////////////////////////////////////////////////////////////////////////////
static void Parse(const FArgument& Argument, void* Out, FStringView Input)
{
	static_assert(sizeof(PTRINT) == sizeof(ParseFunc*));
	auto* Parser = (ParseFunc*)(Argument.Inner);
	return Parser(Out, Input);
}

////////////////////////////////////////////////////////////////////////////////
template <typename Type>
static FArgument MakeArgument(FStringView Name, FStringView Desc)
{
	static_assert(alignof(Type) <= FArgument::ValueAlign);
	static_assert(sizeof(Type) <= FArgument::ValueSize);
	return { Name, Desc, PTRINT(TArgumentParse<Type>::GetParser()) };
}

#define TARGUMENT_IMPL(t) \
	template <> FArgument TArgument<t>(FStringView Name, FStringView Desc) { \
		return MakeArgument<t>(Name, Desc); \
	}
	TARGUMENT_IMPL(FStringView)
	TARGUMENT_IMPL(bool)
	TARGUMENT_IMPL(uint8)	TARGUMENT_IMPL(int8)
	TARGUMENT_IMPL(uint16)	TARGUMENT_IMPL(int16)
	TARGUMENT_IMPL(uint32)	TARGUMENT_IMPL(int32)
	TARGUMENT_IMPL(uint64)	TARGUMENT_IMPL(int64)
	TARGUMENT_IMPL(float)	TARGUMENT_IMPL(double)
#undef TARGUMENT_IMPL



////////////////////////////////////////////////////////////////////////////////
FArgumentSet::FArgumentSet(std::initializer_list<FArgument> InArguments)
: Arguments(InArguments)
{
}

////////////////////////////////////////////////////////////////////////////////
FArgumentSet::operator FArgument () const
{
	return { FStringView(), FStringView(), 0 - PTRINT(this) };
}



////////////////////////////////////////////////////////////////////////////////
FContext::FContext(const FArguments& InArguments, int32 ArgC, const TCHAR* const* ArgV)
{
	SetArguments(InArguments);

	auto NextArg = [&ArgC, &ArgV, i=int32(1), Pending=FStringView()] () mutable -> FStringView
	{
		ArgC--;
		if (auto Ret = Pending; !Ret.IsEmpty())
		{
			Pending.Reset();
			return Ret.Mid(Ret[0] == '=');
		}
		const TCHAR* Arg = ArgV[i++];
		if (Arg[0] != '-')
		{
			return FStringView(Arg);
		}
		const TCHAR* End = Arg;
		for (uint32 c = *End; c == '-' || uint32('z' - (c | 0x20)) <= 26; c = *++End);
		Pending = FStringView(End);
		ArgC += (Pending.IsEmpty() == false);
		return FStringView(Arg, int32(ptrdiff_t(End - Arg)));
	};

	int32 PendingOptional = -1;
	auto HandleOptional = [this, &PendingOptional] (FStringView Arg)
	{
		if (int32 i = PendingOptional; i >= 0)
		{
			if (Arg.IsEmpty() || Arg[0] == '-')
			{
				FArgument Argument = Arguments[i];
				FatalError<Error::FMissingValue>(Argument.Name);
			}
			Parse(Arguments[i], Values[i].Buffer, Arg);
			SetMask |= 1ull << uint64(i);
			PendingOptional = -1;
			return;
		}

		for (int32 i = 0, n = Arguments.Num(); i < n; ++i)
		{
			FArgument& Argument = Arguments[i];
			if (Argument.Name != Arg)
			{
				continue;
			}

			if (Argument.Inner != 0)
			{
				PendingOptional = i;
				return;
			}

			Values[i].Buffer[0] = 1;
			SetMask |= 1ull << uint64(i);
			return;
		}

		FatalError<Error::FUnknownKey>(Arg);
	};

	int32 PositionalIndex = -1;
	auto HandlePositional = [this, &PositionalIndex] (FStringView Arg)
	{
		for (;;)
		{
			++PositionalIndex;
			if (PositionalIndex >= Arguments.Num())
			{
				FatalError<Error::FUnexpectedValue>(Arg);
			}

			if (Arguments[PositionalIndex].Name[0] != '-')
			{
				break;
			}
		}

		FArgument& Argument = Arguments[PositionalIndex];
		if (Argument.Inner == 0)
		{
			FatalError<Error::FBoolPositional>(Arg);
		}

		Parse(Argument, Values[PositionalIndex].Buffer, Arg);
		SetMask |= 1ull << uint64(PositionalIndex);
	};

	Values.SetNum(Arguments.Num());

	while (ArgC > 1)
	{
		FStringView Arg = NextArg();
		if (Arg == TEXT("--"))
		{
			break;
		}

		(PendingOptional >= 0 || Arg[0] == '-')
			? HandleOptional(Arg)
			: HandlePositional(Arg);
	}

	if (PendingOptional >= 0)
	{
		FArgument Argument = Arguments[PendingOptional];
		FatalError<Error::FMissingValue>(Argument.Name);
	}
}

////////////////////////////////////////////////////////////////////////////////
void FContext::SetArguments(const FArguments& Input)
{
	for (const FArgument& Item : Input)
	{
		if (Item.Inner >= 0)
		{
			Arguments.Add(Item);
			continue;
		}

		const auto* Set = (FArgumentSet*)(0 - Item.Inner);
		SetArguments(Set->Arguments);
	}
}

////////////////////////////////////////////////////////////////////////////////
uint32 FContext::FindArgument(FStringView Name) const
{
	for (const FArgument& Argument : Arguments)
	{
		if (Argument.Name == Name)
		{
			return uint32(ptrdiff_t(&Argument - Arguments.GetData()));
		}
	}

	FatalError<Error::FUnknownKey>(Name);
}

////////////////////////////////////////////////////////////////////////////////
[[noreturn]] void FContext::OnMissingValue(FStringView Name) const
{
	FatalError<Error::FMissingValue>(Name);
}

////////////////////////////////////////////////////////////////////////////////
[[noreturn]] void FContext::Abort(const TCHAR* Reason) const
{
	FatalError<Error::FCommandAbort>(Reason);
}

////////////////////////////////////////////////////////////////////////////////
template <typename Lambda>
static void IterArguments(const FArguments& Arguments, const Lambda& Callback)
{
	for (const FArgument& Argument : Arguments)
	{
		if (Argument.Inner >= 0)
		{
			Callback(Argument);
			continue;
		}

		const auto* Set = (FArgumentSet*)(0 - Argument.Inner);
		IterArguments(Set->Arguments, Callback);
	}
}



////////////////////////////////////////////////////////////////////////////////
FCommand::FCommand(
	EntryFunc* InEntry,
	FStringView InName,
	FStringView InDesc,
	FArguments&& InArguments)
{
	Initialize(InEntry, InName, InDesc, MoveTemp(InArguments));
	FCommand::Head() = this;
}

////////////////////////////////////////////////////////////////////////////////
void FCommand::Initialize(
	EntryFunc* InEntry,
	FStringView InName,
	FStringView InDesc,
	FArguments&& InArguments)
{
	Entry = InEntry;
	Name = InName;
	Desc = InDesc;
	Arguments = InArguments;
}

////////////////////////////////////////////////////////////////////////////////
FCommand*& FCommand::Head()
{
	static FCommand* Instance;
	return Instance;
}

////////////////////////////////////////////////////////////////////////////////
FContext FCommand::GetContext(int32 ArgC, const TCHAR* const* ArgV) const
{
	return FContext(Arguments, ArgC, ArgV);
}

////////////////////////////////////////////////////////////////////////////////
void FCommand::Usage() const
{
	if (!Desc.IsEmpty())
	{
		WriteLine(TEXT("%s"), *WriteToString<128>(Desc));
		WriteLine(TEXT(""));
	}

	bool bHasOpts = false;

	TStringBuilder<128> UsageLine;
	UsageLine << TEXT("  ");
	UsageLine << Name;
	IterArguments(Arguments, [&bHasOpts, &UsageLine] (const FArgument& Argument)
	{
		if (Argument.Name.StartsWith('-'))
		{
			bHasOpts = true;
			return;
		}

		UsageLine << TEXT(" <");
		UsageLine << Argument.Name;
		UsageLine << TEXT(">");
	});

	if (bHasOpts)
	{
		UsageLine << TEXT(" [Options]");
	}

	WriteLine(TEXT("Usage:"));
	WriteLine(TEXT("  %s"), *UsageLine);

	if (!bHasOpts)
	{
		return;
	}

	WriteLine(TEXT(""));
	WriteLine(TEXT("Options:"));
	IterArguments(Arguments, [] (const FArgument& Argument)
	{
		if (!Argument.Name.StartsWith('-'))
		{
			return;
		}

		const TCHAR* Suffix = (Argument.Inner != 0) ? TEXT("=<value>") : TEXT("");
		WriteLine(TEXT( "  %s%s"), Argument.Name.GetData(), Suffix);
		if (!Argument.Desc.IsEmpty())
		{
			WriteLine(TEXT("    %s"), Argument.Desc.GetData());
			WriteLine(TEXT(""));
		}
	});
}

////////////////////////////////////////////////////////////////////////////////
int32 FCommand::Call(int32 ArgC, const TCHAR* const* ArgV)
{
	for (int32 i = 1; i < ArgC; ++i)
	{
		if (IsAnAstleySingleFrom91(ArgV[i]))
		{
			Usage();
			return 126;
		}
	}

	FContext Context = GetContext(ArgC, ArgV);
	return Entry(Context);
}

////////////////////////////////////////////////////////////////////////////////
int32 FCommand::Main(int32 ArgC, const TCHAR* const* ArgV)
{
#if PLATFORM_WINDOWS
	::SetUnhandledExceptionFilter([] (EXCEPTION_POINTERS* Pointers) -> LONG
	{
		uint32 Code = Pointers->ExceptionRecord->ExceptionCode;
		void* Where = Pointers->ExceptionRecord->ExceptionAddress;
		WriteLine(TEXT("ERROR: An exceptional circumstance occurred; %08x @ %p"), Code, Where);
		check(false);
		return EXCEPTION_CONTINUE_SEARCH;
	});
#endif

	try
	{
		return MainInner(ArgC, ArgV);
	}
 	catch (const Error::FBase& Err)
	{
		WriteLine(TEXT("ERROR: %s : %s"), Err.What, *(Err.Why));
	}

	return 125;
}

////////////////////////////////////////////////////////////////////////////////
int32 FCommand::MainInner(int32 ArgC, const TCHAR* const* ArgV)
{
	if ((ArgC < 2) || IsAnAstleySingleFrom91(ArgV[1]))
	{
		WriteLine(TEXT("Available commands:"));
		for (FCommand* Cmd = Head(); Cmd != nullptr; Cmd = Cmd->Next)
		{
			WriteLine(TEXT("  %-12s %s"), Cmd->Name.GetData(), Cmd->Desc.GetData());
		}
		return 127;
	}

	FStringView Action = ArgV[1];
	for (FCommand* Cmd = Head(); Cmd != nullptr; Cmd = Cmd->Next)
	{
		if (Cmd->Name == Action)
		{
			return Cmd->Call(ArgC - 1, ArgV + 1);
		}
	}

	FatalError<Error::FCommandNotFound>(Action);
}



////////////////////////////////////////////////////////////////////////////////
void CommandTest()
{
	auto Entry = [] (const FContext&) { return 0; };
	FCommand Command;
	Command.Initialize(Entry, TEXT("name"), TEXT("desc"), {
		TArgument<FStringView>(TEXT("pos0"), TEXT("description")),
		TArgument<FStringView>(TEXT("-string"), TEXT("description")),
		TArgument<FStringView>(TEXT("pos1"), TEXT("description")),
		TArgument<FStringView>(TEXT("pos2"), TEXT("description")),
		TArgument<bool>(TEXT("-bool"), TEXT("description")),
	});

	using TestArgs = TArray<const TCHAR*>;

	auto TestContext = [&] (TestArgs Args)
	{
		Args.Insert(TEXT("$"), 0);
		return Command.GetContext(Args.Num(), Args.GetData());
	};

	// optional
	struct {
		TestArgs	Args;
		FStringView	ExpectString;
		bool		ExpectBool;
	} TestCases0[] = {
		{ {TEXT("-bool")}, TEXT(""), true },
		{ {TEXT("-string=abc")}, TEXT("abc"), false },
		{ {TEXT("-bool"), TEXT("-string=a c")}, TEXT("a c"), true },
		{ {TEXT("-string"), TEXT("abc")}, TEXT("abc"), false },
		{ {TEXT("-string"), TEXT("a=c"), TEXT("-bool")}, TEXT("a=c"), true },
		{ {TEXT("-string=a=c")}, TEXT("a=c"), false },
	};
	for (auto& TestCase : TestCases0)
	{
		FContext Context = TestContext(MoveTemp(TestCase.Args));
		check(Context.Get<bool>(TEXT("-bool")) == TestCase.ExpectBool);
		check(Context.Get<FStringView>(TEXT("-string")) == TestCase.ExpectString);
	}

	for (TestArgs TestCase : {
			TestArgs{ TEXT("-string") },
			TestArgs{ TEXT("-string"), TEXT("-a9e") },
		})
	{
		try { TestContext(MoveTemp(TestCase)); }
		catch (const Error::FMissingValue&) { continue; }
		check(false);
	}

	// positional
	for (TestArgs TestCase : {
			TestArgs{ TEXT("pos0") },
			TestArgs{ TEXT("pos0"), TEXT("pos1") },
			TestArgs{ TEXT("pos0"), TEXT("pos1"), TEXT("pos2") },
		})
	{
		try { TestContext(MoveTemp(TestCase)); }
		catch (...) { check(false); }
	}

	if (bool bTestPass = false; true)
	{
		try { TestContext(TestArgs{TEXT("a"), TEXT("b"), TEXT("b"), TEXT("POP!")}); }
		catch (const Error::FUnexpectedValue&) { bTestPass = true; }
		check(bTestPass);
	}
}

} // namespace UE::IoStore::Tool
