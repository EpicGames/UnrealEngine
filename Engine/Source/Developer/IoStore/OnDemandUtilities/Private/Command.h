// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include <Containers/Array.h>
#include <Containers/StringConv.h>
#include <Containers/StringView.h>
#include <String/LexFromString.h>

#include <initializer_list>

namespace UE::IoStore::Tool {

////////////////////////////////////////////////////////////////////////////////
struct FArgument
{
	static const uint32 ValueAlign = 16;
	static const uint32 ValueSize = 16;
	FStringView			Name;
	FStringView			Desc;
	PTRINT				Inner;
};

using FArguments = TArray<FArgument>;

////////////////////////////////////////////////////////////////////////////////
struct FArgumentSet
{
				FArgumentSet(std::initializer_list<FArgument>);
				operator FArgument () const;
	FArguments	Arguments;
};

////////////////////////////////////////////////////////////////////////////////
template <typename T>
concept CArgTypes =
	std::is_same_v<T, bool>
	|| std::is_integral_v<T>
	|| std::is_floating_point_v<T>
	|| std::is_same_v<T, FStringView>;


////////////////////////////////////////////////////////////////////////////////
template <CArgTypes Type> FArgument TArgument(FStringView Name, FStringView Desc);

#define TARGUMENT_DECL(t) \
	template <> FArgument TArgument<t>(FStringView Name, FStringView Desc);

TARGUMENT_DECL(FStringView)
TARGUMENT_DECL(bool)
TARGUMENT_DECL(uint8)	TARGUMENT_DECL(int8)
TARGUMENT_DECL(uint16)	TARGUMENT_DECL(int16)
TARGUMENT_DECL(uint32)	TARGUMENT_DECL(int32)
TARGUMENT_DECL(uint64)	TARGUMENT_DECL(int64)
TARGUMENT_DECL(float)	TARGUMENT_DECL(double)
#undef TARGUMENT_DECL

////////////////////////////////////////////////////////////////////////////////
class FContext
{
public:
	template <CArgTypes Type> Type	Get(FStringView Name) const;
	template <CArgTypes Type> Type	Get(FStringView Name, Type Default) const;
	[[noreturn]] void				Abort(const TCHAR* Reason) const;

private:
	struct alignas(FArgument::ValueAlign) FValue
	{
		uint8			Buffer[FArgument::ValueSize] = {};
	};

	friend				class FCommand;
						FContext(const FArguments& InArguments, int32 ArgC, const TCHAR* const* ArgV);
	void				SetArguments(const FArguments& Input);
	uint32				FindArgument(FStringView Name) const;
	[[noreturn]] void	OnMissingValue(FStringView Name) const;
	bool				IsSet(uint32 Index) const { return !!((1ull << Index) & SetMask); }
	FArguments			Arguments;
	TArray<FValue>		Values;
	uint64				SetMask = 0;
};

////////////////////////////////////////////////////////////////////////////////
template <CArgTypes Type>
Type FContext::Get(FStringView Name) const
{
	if (Name.StartsWith(TEXT('-')))
	{
		return Get(Name, Type{});
	}

	uint32 Index = FindArgument(Name);
	if (IsSet(Index))
	{
		return *(Type*)(Values[Index].Buffer);
	}

	OnMissingValue(Name);
}

////////////////////////////////////////////////////////////////////////////////
template <CArgTypes Type>
Type FContext::Get(FStringView Name, Type Default) const
{
	uint32 Index = FindArgument(Name);
	return IsSet(Index) ? *(Type*)(Values[Index].Buffer) : Default;
}



////////////////////////////////////////////////////////////////////////////////
class FCommand
{
public:
	using				EntryFunc = int32 (const FContext&);
						FCommand(EntryFunc* InEntry, FStringView InName, FStringView InDesc, FArguments&& InArguments);
	static int32		Main(int32 ArgC, const TCHAR* const* ArgV);

private:
	friend void			CommandTest();
						FCommand() = default;
	void				Initialize(EntryFunc* InEntry, FStringView InName, FStringView InDesc, FArguments&& InArguments);
	FContext			GetContext(int32 ArgC, const TCHAR* const* ArgV) const;
	static FCommand*&	Head();
	static int32		MainInner(int32 ArgC, const TCHAR* const* ArgV);
	void				Usage() const;
	int32				Call(int32 ArgC, const TCHAR* const* ArgV);
	FCommand*			Next = FCommand::Head();
	FStringView			Name;
	FStringView			Desc;
	FArguments			Arguments;
	EntryFunc*			Entry;
};

} // namespace UE::IoStore::Tool
