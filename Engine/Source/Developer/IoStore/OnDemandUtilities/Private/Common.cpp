// Copyright Epic Games, Inc. All Rights Reserved.

#include "Common.h"
#include "Command.h"

#include "Misc/KeyChainUtilities.h"

namespace UE::IoStore::Tool::Common
{

////////////////////////////////////////////////////////////////////////////////
FKeyChain LoadCryptoKeys(const FContext& Context)
{
	FKeyChain Ret;

	const FString Path = FString(Context.Get<FStringView>(TEXT("-CryptoKeys")));
	if (Path.IsEmpty())
	{
		return Ret;
	}

	KeyChainUtilities::LoadKeyChainFromFile(*Path, Ret);

	return Ret;
}

} // namespace UE::IoStore::Tool::Common
