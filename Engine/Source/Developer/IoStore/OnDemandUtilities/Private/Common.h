// Copyright Epic Games, Inc. All Rights Reserved.

#include "Containers/StringView.h"

namespace UE::IoStore::Tool { class FContext; }
struct FKeyChain;

namespace UE::IoStore::Tool::Common
{

////////////////////////////////////////////////////////////////////////////////
FKeyChain LoadCryptoKeys(const FContext& Context);

} // namespace UE::IoStore::Tool::Common
