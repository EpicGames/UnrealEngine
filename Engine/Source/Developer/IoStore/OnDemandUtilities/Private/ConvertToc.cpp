// Copyright Epic Games, Inc. All Rights Reserved.

#include "Command.h"

#include "HAL/FileManager.h"
#include "Hash/ShaderHash.h"
#include "IO/IoChunkId.h"
#include "IO/IoContainerHeader.h"
#include "IO/IoContainerMeta.h"
#include "IO/IoStoreOnDemand.h"
#include "IO/OnDemandToc.h"
#include "IO/PackageId.h"
#include "IO/Serialization/OnDemandContainerToc.h"
#include "Misc/FileHelper.h"
#include "Misc/Paths.h"
#include "Misc/SecureHash.h"
#include "Misc/StringBuilder.h"
#include "Runtime/Experimental/IoStore/OnDemand/Private/OnDemandIoStore.h"
#include "Serialization/CompactBinarySerialization.h"
#include "Serialization/CompactBinaryWriter.h"
#include "Serialization/LargeMemoryReader.h"

namespace UE::IoStore::Tool
{

////////////////////////////////////////////////////////////////////////////////
static FIoChunkId GetPackageChunkId(const FIoChunkId& ChunkId, FPackageId& OutPackageId)
{
	FMemory::Memcpy(&OutPackageId, &ChunkId, sizeof(FPackageId));

	if (ChunkId.GetChunkType() == EIoChunkType::ExportBundleData)
	{
		return ChunkId;
	}

	return CreatePackageDataChunkId(OutPackageId);
}

////////////////////////////////////////////////////////////////////////////////
static FIoChunkId GetPackageChunkId(const FIoChunkId& ChunkId)
{
	FPackageId Unused;
	return GetPackageChunkId(ChunkId, Unused);
}

////////////////////////////////////////////////////////////////////////////////
class FContainerMeta
{
public:
	FUtf8StringView			GetFilename(
								const FIoChunkId& ChunkId,
								FUtf8StringBuilderBase& OutFilename,
								FUtf8StringView& OutContainerName);
	FUtf8StringView			GetFilename(
								const FIoChunkId& ChunkId,
								FUtf8StringBuilderBase& OutFilename);
	static FContainerMeta	Load(const FString& Directory);

private:
	TArray<FIoContainerMetaReader> Readers;
};

////////////////////////////////////////////////////////////////////////////////
FUtf8StringView FContainerMeta::GetFilename(
	const FIoChunkId& ChunkId,
	FUtf8StringBuilderBase& OutFilename,
	FUtf8StringView& OutContainerName)
{
	OutFilename.Reset();
	for (FIoContainerMetaReader& Reader : Readers)
	{
		FUtf8StringView Filename = Reader.GetFilename(ChunkId, OutFilename, OutContainerName);
		if (!Filename.IsEmpty())
		{
			return Filename;
		}
	}

	return FUtf8StringView();
}

FUtf8StringView FContainerMeta::GetFilename(
	const FIoChunkId& ChunkId,
	FUtf8StringBuilderBase& OutFilename)
{
	FUtf8StringView Unused;
	return GetFilename(ChunkId, OutFilename, Unused);
}

FContainerMeta FContainerMeta::Load(const FString& Directory)
{
	FContainerMeta	OutMeta;
	TArray<FString> MetaFiles;

	IFileManager& Ifm = IFileManager::Get();
	Ifm.FindFiles(MetaFiles, *Directory, TEXT(".umeta"));

	for (const FString& Filename : MetaFiles)
	{
		const FString	FullPath = Directory / Filename;
		const bool		bMemoryMap = false;

		if (TIoStatusOr<FIoContainerMetaReader> Status = FIoContainerMetaReader::Load(FullPath, bMemoryMap); Status.IsOk())
		{
			OutMeta.Readers.Add(Status.ConsumeValueOrDie());
		}
		else
		{
			UE_LOGF(LogIoStoreOnDemand, Warning, "Failed to load container meta '%ls'", *FullPath);
		}
	}

	return OutMeta;
}

////////////////////////////////////////////////////////////////////////////////
TArray<UE::IoStore::Serialization::V2::FOnDemandTocReader> LoadContainerTocs(const FString& PathOrDir)
{
	using namespace UE::IoStore::Serialization;
	using namespace UE::IoStore::Serialization::V2;

	IFileManager& Ifm = IFileManager::Get();

	TArray<FString> FilePaths;
	if (Ifm.FileExists(*PathOrDir))
	{
		FilePaths.Add(PathOrDir);
	}
	else
	{
		TArray<FString> Filenames;
		Ifm.FindFiles(Filenames, *PathOrDir, FOnDemandFileExt::Toc);

		for (const FString& Filename : Filenames)
		{
			FilePaths.Add(PathOrDir / Filename);
		}
	}

	TArray<FOnDemandTocReader> Readers;
	for (const FString& FilePath : FilePaths)
	{
		TIoStatusOr<FOnDemandTocReader> MaybeReader = FOnDemandTocReader::Read(FilePath);
		if (MaybeReader.IsOk())
		{
			UE_LOGF(LogIoStoreOnDemand, Display, "Loaded '%ls'", *FilePath);
			Readers.Add(MaybeReader.ConsumeValueOrDie());
		}
		else
		{
			UE_LOGF(LogIoStoreOnDemand, Error, "Failed to read on-demand TOC '%ls', reason: %ls",
				*PathOrDir, *MaybeReader.Status().ToString());
		}
	}

	return Readers;
}

////////////////////////////////////////////////////////////////////////////////
static int32 ConvertTocCommandEntry(const FContext& Context)
{
	using namespace UE::IoStore::Serialization;
	using namespace UE::IoStore::Serialization::V2;
	using FChunkEntry = UE::IoStore::Serialization::V2::FOnDemandChunkEntry;

	GPrintLogCategory = false;
	GPrintLogVerbosity = false;
	GPrintLogTimes = ELogTimes::None;

	FString FilePath = FString(Context.Get<FStringView>(TEXT("-Path")));
	if (FilePath.IsEmpty())
	{
		UE_LOGF(LogIoStoreOnDemand, Error, "Invalid filename");
		return -1;
	}

	FString OutFilePath = FString(Context.Get<FStringView>(TEXT("-Out")));
	if (OutFilePath.IsEmpty())
	{
		UE_LOGF(LogIoStoreOnDemand, Error, "Invalid JSON output filename");
		return -1;
	}

	const FString Ext = FPaths::GetExtension(OutFilePath).ToLower();
	if (Ext != TEXT("json") && Ext != TEXT("csv"))
	{
		UE_LOGF(LogIoStoreOnDemand, Error, "Invalid output file extension");
		return -1;
	}

	TArray<FOnDemandTocReader> Readers = LoadContainerTocs(FilePath);
	if (Readers.IsEmpty())
	{
		UE_LOGF(LogIoStoreOnDemand, Error, "Failed to find container TOC file(s)");
		return -1;
	}

	const bool bIncludePackageStore = Context.Get<bool>(TEXT("-IncludePackageStoreEntries"), false);
	const bool bExcludeChunkEntries = Context.Get<bool>(TEXT("-ExcludeChunkEntries"), false);

	IFileManager& Ifm = IFileManager::Get();
	FContainerMeta ContainerMeta = FContainerMeta::Load(FPaths::GetPath(FilePath));

	if (Ext == TEXT("json"))
	{
		FCbWriter Writer;
		Writer.BeginArray();
		for (FOnDemandTocReader& Reader : Readers)
		{
			Writer.BeginObject();
			{
				Writer.BeginObject(UTF8TEXTVIEW("Header"));
				Writer.AddString(UTF8TEXTVIEW("CreationTime"), Reader.CreationTime().ToString());
				Writer.AddString(UTF8TEXTVIEW("BuildVersion"), Reader.BuildVersion());
				Writer.AddString(UTF8TEXTVIEW("TargetPlatform"), Reader.TargetPlatform());
				Writer.AddString(UTF8TEXTVIEW("ChunksDirectory"), Reader.ChunksDirectory());
				Writer.EndObject();

				Writer.BeginArray(UTF8TEXTVIEW("Containers"));
				for (const FOnDemandContainerEntry& ContainerEntry : Reader.Containers())
				{
					FOnDemandTocStorage Storage;
					TIoStatusOr<FOnDemandContainerTocView> MaybeContainerView =
						Reader.ReadContainer(ContainerEntry, Storage, EOnDemandTocReaderOptions::None);

					if (MaybeContainerView.IsOk() == false)
					{
						UE_LOGF(LogIoStoreOnDemand, Error, "Failed to read on-demand container TOC '%ls', reason: %ls",
							*FilePath, *MaybeContainerView.Status().ToString());
						return -1;
					}

					FOnDemandContainerTocView ContainerView = MaybeContainerView.ConsumeValueOrDie();

					Writer.BeginObject();
					{
						Writer.AddString(UTF8TEXTVIEW("ContainerName"), ContainerView.Header.ContainerName());
						Writer.AddString(UTF8TEXTVIEW("ContainerId"), LexToString(ContainerView.Header.ContainerId()));
						Writer.AddInteger(UTF8TEXTVIEW("ContainerFlags"), uint32(ContainerView.Header.ContainerFlags()));
						Writer.AddString(UTF8TEXTVIEW("EncryptionKeyGuid"), ContainerView.Header.EncryptionKeyGuid().ToString());
						Writer.AddInteger(UTF8TEXTVIEW("BlockSize"), ContainerView.Header.BlockSize());

						if (bExcludeChunkEntries == false)
						{
							TUtf8StringBuilder<512> Sb;
							Writer.BeginArray(UTF8TEXTVIEW("PartitionEntries"));
							for (const FOnDemandPartitionEntry& Partition : ContainerView.PartitionEntries)
							{
								Writer.BeginObject();
								Writer.AddString(UTF8TEXTVIEW("Hash"), LexToString(Partition.Hash));
								Writer.AddInteger(UTF8TEXTVIEW("Size"), Partition.Size);
								Writer.EndObject();
							}
							Writer.EndArray();

							Writer.BeginArray(UTF8TEXTVIEW("ChunkEntries"));
							for (int32 Idx = 0; const FIoChunkId& ChunkId : ContainerView.ChunkIds)
							{
								FPackageId			PackageId;
								FUtf8StringView		Filename = ContainerMeta.GetFilename(GetPackageChunkId(ChunkId, PackageId), Sb);
								const FChunkEntry&	ChunkEntry = ContainerView.ChunkEntries[Idx++];
								Writer.BeginObject();
								{
									Writer.AddString(UTF8TEXTVIEW("Filename"), Filename);
									Writer.AddString(UTF8TEXTVIEW("ChunkId"), LexToString(ChunkId));
									Writer.AddString(UTF8TEXTVIEW("PackageId"), LexToString(PackageId));
									Writer.AddString(UTF8TEXTVIEW("Hash"), LexToString(ChunkEntry.Hash));
									Writer.AddInteger(UTF8TEXTVIEW("RawSize"), ChunkEntry.RawSize);
									Writer.AddInteger(UTF8TEXTVIEW("EncodedSize"), ChunkEntry.EncodedSize);
									Writer.AddInteger(UTF8TEXTVIEW("BlockOffset"), ChunkEntry.BlockInfo.Offset());
									Writer.AddInteger(UTF8TEXTVIEW("BlockCount"), ChunkEntry.BlockInfo.Count());
									if (ChunkEntry.BlockInfo.Count() == 1)
									{
										Writer.AddInteger(UTF8TEXTVIEW("BlockSize"), ChunkEntry.BlockInfo.Size());
										Writer.AddInteger(UTF8TEXTVIEW("BlockHash"), ChunkEntry.BlockInfo.Hash());
									}
								}
								Writer.EndObject();
							}
							Writer.EndArray();
						}
						
						if (bIncludePackageStore)
						{
							FLargeMemoryReader Ar(
								ContainerView.ContainerHeaderChunk.GetData(),
								ContainerView.ContainerHeaderChunk.GetSize());
							
							FIoContainerHeader ContainerHeader;
							Ar << ContainerHeader;

							if (Ar.IsError() || Ar.IsCriticalError())
							{
								UE_LOGF(LogIoStoreOnDemand, Error, "Failed to serialize container header");
							}
							else
							{
								TConstArrayView<FFilePackageStoreEntry> PackageStoreEntries = MakeArrayView(
									reinterpret_cast<const FFilePackageStoreEntry*>(
										ContainerHeader.StoreEntries.GetData()),
										ContainerHeader.PackageIds.Num());

								Writer.BeginArray(UTF8TEXTVIEW("PackageStoreEntries"));
								for (int32 Index = 0; const FPackageId& PackageId : ContainerHeader.PackageIds)
								{
									const FFilePackageStoreEntry& Entry = PackageStoreEntries[Index++];

									Writer.BeginObject();
									{
										Writer.AddString(UTF8TEXTVIEW("PackageId"), LexToString(PackageId));
										Writer.BeginArray(UTF8TEXTVIEW("ImportedPackageIds"));
										{
											for (const FPackageId& Import : Entry.ImportedPackages)
											{
												Writer << LexToString(Import);
											}
										}
										Writer.EndArray();
										Writer.BeginArray(UTF8TEXTVIEW("ShaderMapHashes"));
										{
											for (const FShaderHash& Hash : Entry.ShaderMapHashes)
											{
												Writer << LexToString(Hash);
											}
										}
										Writer.EndArray();
									}
									Writer.EndObject();
								}
								Writer.EndArray();
							}
						}
					}
					Writer.EndObject();
				}
				Writer.EndArray();
			}
			Writer.EndObject();
		}
		Writer.EndArray();

		TStringBuilder<4096> Json;
		CompactBinaryToJson(Writer.Save().AsArray(), Json);

		if (!FFileHelper::SaveStringToFile(Json.ToView(), *OutFilePath, FFileHelper::EEncodingOptions::ForceUTF8WithoutBOM))
		{
			UE_LOGF(LogIoStoreOnDemand, Error, "Failed to save JSON '%ls'", *OutFilePath);
			return -1;
		}
	}
	else
	{
		TUniquePtr<FArchive> Ar(Ifm.CreateFileWriter(*OutFilePath));
		if (!Ar.IsValid())
		{
			UE_LOGF(LogIoStoreOnDemand, Error, "Failed to open '%ls' for writing", *OutFilePath);
			return -1;
		}

		Ar->Logf(TEXT("Platform, ContainerName, ChunkId, PackageId, Hash, Size, CompressedSize, Filename"));
		for (FOnDemandTocReader& Reader : Readers)
		{
			for (const FOnDemandContainerEntry& ContainerEntry : Reader.Containers())
			{
				FOnDemandTocStorage Storage;
				TIoStatusOr<FOnDemandContainerTocView> MaybeContainerView =
					Reader.ReadContainer(ContainerEntry, Storage, EOnDemandTocReaderOptions::None);

				if (MaybeContainerView.IsOk() == false)
				{
					UE_LOGF(LogIoStoreOnDemand, Error, "Failed to read on-demand container TOC '%ls', reason: %ls",
						*FilePath, *MaybeContainerView.Status().ToString());
					return -1;
				}

				FOnDemandContainerTocView ContainerView = MaybeContainerView.ConsumeValueOrDie();

				TUtf8StringBuilder<512> Sb;
				for (int32 Idx = 0; const FIoChunkId& ChunkId : ContainerView.ChunkIds)
				{
					FPackageId			PackageId;
					FUtf8StringView		Filename = ContainerMeta.GetFilename(GetPackageChunkId(ChunkId, PackageId), Sb);
					const FChunkEntry&	ChunkEntry = ContainerView.ChunkEntries[Idx++];
					FString				ContainerFullName = FString(ContainerView.Header.ContainerName());

					FString PakChunk, Platform;
					ContainerFullName.Split(TEXT("-"), &PakChunk, &Platform, ESearchCase::CaseSensitive, ESearchDir::FromEnd);

					Ar->Logf(TEXT("%s, %s, %s, %s, %s, %u, %u, %s"),
						*Platform,
						*PakChunk,
						*LexToString(ChunkId),
						*LexToString(PackageId),
						*LexToString(ChunkEntry.Hash),
						ChunkEntry.RawSize,
						ChunkEntry.EncodedSize,
						*FString(Filename));
				}
			}
		}
	}

	UE_LOGF(LogIoStoreOnDemand, Display, "Saved '%ls'", *OutFilePath);

	return 0;
}

////////////////////////////////////////////////////////////////////////////////
static FCommand ConvertTocCommand(
	ConvertTocCommandEntry,
	TEXT("ConvertToc"),
	TEXT("Converts an on-demand TOC file to JSON or CSV"),
	{
		TArgument<FStringView>(TEXT("-Path"), TEXT("Path to a .uondemandtoc file or a directory")),
		TArgument<bool>(TEXT("-IncludePackageStoreEntries"), TEXT("Whether to include package store information.")),
		TArgument<bool>(TEXT("-ExcludeChunkEntries"), TEXT("Whether to exclude chunk entry information.")),
		TArgument<FStringView>(TEXT("-Out"), TEXT("Output file path"))
	}
);

} // namespace UE::IoStore::Tool
