// Copyright Epic Games, Inc. All Rights Reserved.

#include "Command.h"

#include "HAL/FileManager.h"
#include "IO/IoStore.h"
#include "IO/IoStoreOnDemand.h"
#include "Misc/Paths.h"
#include "S3/S3Client.h"
#include "Serialization/CompactBinarySerialization.h"
#include "Serialization/CompactBinaryWriter.h"
#include "Serialization/LargeMemoryWriter.h"
#include "Serialization/MemoryReader.h"

namespace UE::IoStore::Tool
{

////////////////////////////////////////////////////////////////////////////////
struct FDownloadParams
{
	FString Directory;
	FString ServiceUrl;
	FString Bucket;
	FString Region; 
	FString AccessKey;
	FString SecretKey;
	FString SessionToken;
	FString CredentialsFile;
	FString CredentialsFileKeyName;
	int32 MaxConcurrentDownloads = 16;
	
	static TIoStatusOr<FDownloadParams> Parse(const TCHAR* CommandLine);
	FIoStatus Validate() const;
};

FIoStatus FDownloadParams::Validate() const
{
	if (!AccessKey.IsEmpty() && SecretKey.IsEmpty())
	{
		return FIoStatus(EIoErrorCode::InvalidParameter, TEXT("Invalid secret key"));
	}
	else if (AccessKey.IsEmpty() && !SecretKey.IsEmpty())
	{
		return FIoStatus(EIoErrorCode::InvalidParameter, TEXT("Invalid access key"));
	}

	if (!CredentialsFile.IsEmpty() && CredentialsFileKeyName.IsEmpty())
	{
		return FIoStatus(EIoErrorCode::InvalidParameter, TEXT("Invalid credential file key name"));
	}

	if (ServiceUrl.IsEmpty() && Region.IsEmpty())
	{
		return FIoStatus(EIoErrorCode::InvalidParameter, TEXT("Service URL or AWS region needs to be specified"));
	}

	return FIoStatus::Ok;
}

////////////////////////////////////////////////////////////////////////////////
FIoStatus DownloadContainerFiles(const FDownloadParams& DownloadParams, const FString& TocPath)
{
	struct FContainerStats
	{
		uint64 TocEntryCount = 0;
		uint64 TocRawSize = 0;
		uint64 RawSize = 0;
		uint64 CompressedSize = 0;
		EIoContainerFlags ContainerFlags = EIoContainerFlags::None;
		FName CompressionMethod;
	};
	TMap<FString, FContainerStats> ContainerSummary;

	const double StartTime = FPlatformTime::Seconds();

	FS3ClientConfig Config;
	Config.ServiceUrl = DownloadParams.ServiceUrl;
	Config.Region = DownloadParams.Region;
	
	FS3ClientCredentials Credentials;
	if (DownloadParams.CredentialsFile.IsEmpty() == false)
	{
		UE_LOGF(LogIas, Display, "Loading credentials file '%ls'", *DownloadParams.CredentialsFile);
		FS3CredentialsProfileStore CredentialsStore = FS3CredentialsProfileStore::FromFile(DownloadParams.CredentialsFile);
		if (CredentialsStore.TryGetCredentials(DownloadParams.CredentialsFileKeyName, Credentials) == false)
		{
			return FIoStatus(EIoErrorCode::InvalidParameter, TEXT("Failed to find valid credentials in credentials file"));
		}
		else
		{
			UE_LOGF(LogIas, Display, "Found credentials for '%ls'", *DownloadParams.CredentialsFileKeyName);
		}
	}
	else
	{
		Credentials = FS3ClientCredentials(DownloadParams.AccessKey, DownloadParams.SecretKey, DownloadParams.SessionToken);
	}

	FS3Client Client(Config, Credentials);

	UE_LOGF(LogIas, Display, "Fetching TOC '%ls/%ls/%ls'", *Client.GetConfig().ServiceUrl, *DownloadParams.Bucket, *TocPath);
	FS3GetObjectResponse TocResponse = Client.GetObject(FS3GetObjectRequest
	{
		DownloadParams.Bucket,
		TocPath
	});

	if (TocResponse.IsOk() == false)
	{
		return FIoStatus(EIoErrorCode::ReadError, TEXT("Failed to fetch TOC"));
	}

	FOnDemandToc OnDemandToc;
	FMemoryReaderView Ar(TocResponse.GetBody().GetView());
	Ar << OnDemandToc;

	if (Ar.IsError())
	{
		return FIoStatus(EIoErrorCode::ReadError, TEXT("Failed to load on demand TOC"));
	}

	FStringView BucketPrefix;
	{
		FStringView TocPathView = TocPath;
		int32 Index = INDEX_NONE;
		if (TocPathView.FindLastChar(TEXT('/'), Index))
		{
			BucketPrefix = TocPathView.Left(Index);
		}
	}

	for (const FOnDemandTocContainerEntry& ContainerEntry : OnDemandToc.Containers)
	{
		TStringBuilder<256> FileTocKey;
		if (BucketPrefix.IsEmpty() == false)
		{
			FileTocKey << BucketPrefix << TEXT("/");
		}
		FileTocKey << ContainerEntry.UTocHash << TEXT(".utoc");

		UE_LOGF(LogIas, Display, "Fetching '%ls/%ls/%ls'", *Client.GetConfig().ServiceUrl, *DownloadParams.Bucket, FileTocKey.ToString());
		FS3GetObjectResponse Response = Client.GetObject(FS3GetObjectRequest
		{
			DownloadParams.Bucket,
			FString(FileTocKey).ToLower()
		});

		if (Response.IsOk() == false)
		{
			return FIoStatus(EIoErrorCode::ReadError, TEXT("Failed to load container .utoc file"));
		}

		const FString UTocPath = FString::Printf(TEXT("%s/%s.utoc"), *DownloadParams.Directory, *ContainerEntry.ContainerName);
		const FString UCasPath = FPaths::ChangeExtension(UTocPath, TEXT(".ucas")); 
		FContainerStats& ContainerStats = ContainerSummary.FindOrAdd(ContainerEntry.ContainerName);
		ContainerStats.TocRawSize = Response.GetBody().GetSize();

		if (TUniquePtr<FArchive> TocFile(IFileManager::Get().CreateFileWriter(*UTocPath)); TocFile.IsValid())
		{
			UE_LOGF(LogIas, Display, "Writing '%ls'", *UTocPath);
			TocFile->Serialize((void*)Response.GetBody().GetData(), Response.GetBody().GetSize());
		}
		else
		{
			return FIoStatus(EIoErrorCode::WriteError, TEXT("Failed to write container .utoc file"));
		}

		FIoStoreTocResource FileToc;
		if (FIoStatus Status = FIoStoreTocResource::Read(*UTocPath, EIoStoreTocReadOptions::ReadAll, FileToc); Status.IsOk() == false)
		{
			return Status;
		}

		const int32 TocEntryCount = int32(FileToc.Header.TocEntryCount);
		const uint64 CompressionBlockSize = FileToc.Header.CompressionBlockSize;
		ContainerStats.TocEntryCount = TocEntryCount;
		ContainerStats.ContainerFlags = FileToc.Header.ContainerFlags; 

		TMap<FIoChunkId, FIoHash> ChunkHashes;
		for (const FOnDemandTocEntry& Entry : ContainerEntry.Entries)
		{
			ChunkHashes.Add(Entry.ChunkId, Entry.Hash);
			ContainerStats.RawSize += Entry.RawSize;
		}

		TArray<int32> SortedIndices;
		for (int32 Idx = 0; Idx < TocEntryCount; ++Idx)
		{
			SortedIndices.Add(Idx);
		}

		Algo::Sort(SortedIndices, [&FileToc](int32 Lhs, int32 Rhs)
		{
			return FileToc.ChunkOffsetLengths[Lhs].GetOffset() < FileToc.ChunkOffsetLengths[Rhs].GetOffset();
		});

		FString ChunksRelativePath = BucketPrefix.IsEmpty()
			? TEXT("chunks")
			: FString::Printf(TEXT("%s/chunks"), *FString(BucketPrefix));

		TUniquePtr<FArchive> CasFile(IFileManager::Get().CreateFileWriter(*UCasPath));
		TArray<uint8> PaddingBuffer;

		for (int32 Idx = 0; Idx < TocEntryCount; ++Idx)
		{
			const int32 SortedIdx = SortedIndices[Idx];
			const FIoChunkId& ChunkId = FileToc.ChunkIds[SortedIdx];
			const FIoOffsetAndLength& OffsetLength = FileToc.ChunkOffsetLengths[SortedIdx];
			const FIoHash ChunkHash = ChunkHashes.FindChecked(ChunkId);
			const FString HashString = LexToString(ChunkHash);
			const int32 FirstBlockIndex = int32(OffsetLength.GetOffset() / CompressionBlockSize);
			const int32 LastBlockIndex = int32((Align(OffsetLength.GetOffset() + OffsetLength.GetLength(), CompressionBlockSize) - 1) / CompressionBlockSize);
			const FIoStoreTocCompressedBlockEntry& FirstBlock = FileToc.CompressionBlocks[FirstBlockIndex];
			const FName CompressionMethod = FileToc.CompressionMethods[FirstBlock.GetCompressionMethodIndex()];

			if (CompressionMethod.IsNone() == false && ContainerStats.CompressionMethod.IsNone())
			{
				ContainerStats.CompressionMethod = CompressionMethod;
			}

			TStringBuilder<256> ChunkKey;
			ChunkKey << ChunksRelativePath
				<< TEXT("/") << HashString.Left(2)
				<< TEXT("/") << HashString
				<< TEXT(".iochunk");

			UE_LOGF(LogIas, Display, "Fetching '%ls/%ls/%ls'", *Client.GetConfig().ServiceUrl, *DownloadParams.Bucket, ChunkKey.ToString());
			FS3GetObjectResponse ChunkResponse = Client.GetObject(FS3GetObjectRequest
			{
				DownloadParams.Bucket,
				FString(ChunkKey).ToLower()
			});

			if (ChunkResponse.IsOk() == false)
			{
				return FIoStatus(EIoErrorCode::ReadError, TEXT("Failed to fetch chunk"));
			}

			const uint64 Padding = FirstBlock.GetOffset() - CasFile->Tell();
			if (Padding > PaddingBuffer.Num())
			{
				PaddingBuffer.SetNumZeroed(int32(Padding));
			}

			if (Padding > 0)
			{
				CasFile->Serialize(PaddingBuffer.GetData(), Padding);
			}

			UE_LOGF(LogIas, Display, "Serializing chunk %d/%d '%ls' -> '%ls' (%llu B)",
				Idx + 1, TocEntryCount, *HashString, *LexToString(ChunkId), ChunkResponse.GetBody().GetSize());

			check(CasFile->Tell() == FirstBlock.GetOffset());
			CasFile->Serialize((void*)ChunkResponse.GetBody().GetData(), ChunkResponse.GetBody().GetSize());
		}

		ContainerStats.CompressedSize = CasFile->Tell();
	}

	{
		const double Duration = FPlatformTime::Seconds() - StartTime;

		UE_LOGF(LogIas, Display, "");
		UE_LOGF(LogIas, Display, "---------------------------------------- Download Summary --------------------------------------");
		UE_LOGF(LogIas, Display, "%-40ls: %ls", TEXT("Service URL"), *DownloadParams.ServiceUrl);
		UE_LOGF(LogIas, Display, "%-40ls: %ls", TEXT("Bucket"), *DownloadParams.Bucket);
		UE_LOGF(LogIas, Display, "%-40ls: %ls", TEXT("TOC"), *TocPath);
		UE_LOGF(LogIas, Display, "%-40ls: %.2lf second(s)", TEXT("Duration"), Duration);
		UE_LOGF(LogIas, Display, "");

		UE_LOGF(LogIas, Display, "%-30ls %10ls %15ls %15ls %15ls %25ls",
			TEXT("Container"), TEXT("Flags"), TEXT("TOC Size (KB)"), TEXT("TOC Entries"), TEXT("Size (MB)"), TEXT("Compressed (MB)"));
		UE_LOGF(LogIas, Display, "-------------------------------------------------------------------------------------------------------------------------");
		
		FContainerStats TotalStats;
		for (const TTuple<FString, FContainerStats>& Kv : ContainerSummary)
		{
			const FString& ContainerName = Kv.Key;
			const FContainerStats& Stats = Kv.Value;

			FString CompressionInfo = "-";

			if (Stats.CompressionMethod != NAME_None)
			{
				const double Percentage = Stats.RawSize > 0 ? (double(Stats.RawSize - Stats.CompressedSize) / double(Stats.RawSize)) * 100.0 : 0.0;
				CompressionInfo = FString::Printf(TEXT("%.2lf (%.2lf%% %s)"),
					(double)Stats.CompressedSize / 1024.0 / 1024.0,
					Percentage,
					*Stats.CompressionMethod.ToString());
			}

			FString ContainerSettings = FString::Printf(TEXT("%s/%s/%s/%s/%s"),
				EnumHasAnyFlags(Stats.ContainerFlags, EIoContainerFlags::Compressed) ? TEXT("C") : TEXT("-"),
				EnumHasAnyFlags(Stats.ContainerFlags, EIoContainerFlags::Encrypted) ? TEXT("E") : TEXT("-"),
				EnumHasAnyFlags(Stats.ContainerFlags, EIoContainerFlags::Signed) ? TEXT("S") : TEXT("-"),
				EnumHasAnyFlags(Stats.ContainerFlags, EIoContainerFlags::Indexed) ? TEXT("I") : TEXT("-"),
				EnumHasAnyFlags(Stats.ContainerFlags, EIoContainerFlags::OnDemand) ? TEXT("O") : TEXT("-"));

			UE_LOGF(LogIas, Display, "%-30ls %10ls %15.2lf %15llu %15.2lf %25ls",
				*ContainerName,
				*ContainerSettings,
				(double)Stats.TocRawSize / 1024.0,
				Stats.TocEntryCount,
				(double)Stats.RawSize / 1024.0 / 1024.0,
				*CompressionInfo);

			TotalStats.TocEntryCount += Stats.TocEntryCount;
			TotalStats.TocRawSize += Stats.TocRawSize;
			TotalStats.RawSize += Stats.RawSize;
			TotalStats.CompressedSize += Stats.CompressedSize;
		}
		UE_LOGF(LogIas, Display, "-------------------------------------------------------------------------------------------------------------------------");
		UE_LOGF(LogIas, Display, "%-30ls %10ls %15.2lf %15llu %15.2lf %25.2lf ",
			TEXT("Total"),
			TEXT(""),
			(double)TotalStats.TocRawSize / 1024.0,
			TotalStats.TocEntryCount,
			(double)TotalStats.RawSize / 1024.0 / 1024.0,
			(double)TotalStats.CompressedSize / 1024.0 / 1024.0);

		UE_LOGF(LogIas, Display, "");
		UE_LOGF(LogIas, Display, "** Flags: (C)ompressed / (E)ncrypted / (S)igned) / (I)ndexed) / (O)nDemand **");
		UE_LOGF(LogIas, Display, "");
	}

	return FIoStatus::Ok;
}

////////////////////////////////////////////////////////////////////////////////
static FDownloadParams BuildDownloadParams(const FContext& Context)
{
	FDownloadParams Ret;

	Ret.Directory				= Context.Get<FStringView>(TEXT("-Directory"),				Ret.Directory);
	Ret.ServiceUrl				= Context.Get<FStringView>(TEXT("-ServiceUrl"),				Ret.ServiceUrl);
	Ret.Bucket					= Context.Get<FStringView>(TEXT("-Bucket"),					Ret.Bucket);
	Ret.Region					= Context.Get<FStringView>(TEXT("-Region"),					Ret.Region);
	Ret.AccessKey				= Context.Get<FStringView>(TEXT("-AccessKey"),				Ret.AccessKey);
	Ret.SecretKey				= Context.Get<FStringView>(TEXT("-SecretKey"),				Ret.SecretKey);
	Ret.SessionToken			= Context.Get<FStringView>(TEXT("-SessionToken"),			Ret.SessionToken);
	Ret.CredentialsFile			= Context.Get<FStringView>(TEXT("-CredentialsFile"),		Ret.CredentialsFile);
	Ret.CredentialsFileKeyName	= Context.Get<FStringView>(TEXT("-CredentialsFileKeyName"),	Ret.CredentialsFileKeyName);

	Ret.MaxConcurrentDownloads	= Context.Get(TEXT("-MaxConcurrentDownloads"),	Ret.MaxConcurrentDownloads);

	if (FIoStatus Validation = Ret.Validate(); !Validation.IsOk())
	{
		FString Reason = Validation.ToString();
		Context.Abort(*Reason);
	}

	return Ret;
}

////////////////////////////////////////////////////////////////////////////////
static int32 DownloadCommandEntry(const Tool::FContext& Context)
{
	FStringView TocPath = Context.Get<FStringView>(TEXT("TocPath"));

	FDownloadParams Params = BuildDownloadParams(Context);

	FIoStatus Status = DownloadContainerFiles(Params, FString(TocPath));
	if (!Status.IsOk())
	{
		FString Reason = Status.ToString();
		Context.Abort(*Reason);
	}

	return 0;
}

////////////////////////////////////////////////////////////////////////////////
extern FArgumentSet S3Arguments;

static FCommand DownloadCommand(
	DownloadCommandEntry,
	TEXT("Download"),
	TEXT("Fetches the cloud-stored contents of a given on-demand TOC"),
	{
		TArgument<FStringView>(TEXT("TocPath"),				TEXT("Bucket-relative path of the TOC to download")),
		TArgument<FStringView>(TEXT("-Directory"),			TEXT("Output directory")),
		TArgument<int32>(TEXT("-MaxConcurrentDownloads"),	TEXT("Number of downloads that happen all at once")),
		S3Arguments,
	}
);

} // namespace UE::IoStore::Tool
