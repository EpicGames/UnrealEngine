// Copyright Epic Games, Inc. All Rights Reserved.

#include "Graph.h"
#include "Command.h"
#include "Common.h"

#include "HAL/FileManager.h"
#include "HAL/Platform.h"
#include "IO/IoContainerHeader.h"
#include "IO/IoContainerId.h"
#include "IO/IoStore.h"
#include "IO/IoStoreOnDemand.h"
#include "IO/PackageId.h"
#include "IO/Serialization/OnDemandContainerToc.h"
#include "Misc/CString.h"
#include "Misc/FileHelper.h"
#include "Misc/KeyChainUtilities.h"
#include "Misc/Paths.h"
#include "Misc/Parse.h"
#include "Serialization/CompactBinary.h"
#include "Serialization/CompactBinaryWriter.h"
#include "Serialization/CompactBinarySerialization.h"
#include "Serialization/LargeMemoryReader.h"

namespace UE::IoStore::Tool
{

////////////////////////////////////////////////////////////////////////////////
FIoStatus ExportGephiCsv(const FPackageGraph& Graph, const FString& OutFilename)
{
	UE_LOGF(LogIoStoreOnDemand, Display, "Exporting CSV with %d nodes and %d edges", Graph.GetNodeCount(), Graph.GetEdgeCount());
	IFileManager& Ifm = IFileManager::Get();
	{
		TUniquePtr<FArchive> Ar(Ifm.CreateFileWriter(*OutFilename));
		if (!Ar.IsValid())
		{
			return FIoStatus(EIoErrorCode::WriteError);
		}

		UE_LOGF(LogIoStoreOnDemand, Display, "Saving '%ls'", *OutFilename);
		Ar->Logf(TEXT("Id, Label"));
		for (int32 Idx = 0; Idx < Graph.GetNodeCount(); ++Idx)
		{
			FPackageGraph::FNode			Node{Idx};
			const FPackageGraph::FNodeData&	Data = Graph.GetNodeData(Node);
			Ar->Logf(TEXT("%d, %s"), Idx, *LexToString(Data.PackageId));
		}

		if (!Ar->Close())
		{
			return FIoStatus(EIoErrorCode::WriteError);
		}
	}

	{
		FString Filename		= FPaths::GetBaseFilename(OutFilename);
		FString Path			= FPaths::GetPath(OutFilename);
		FString EdgesFilename	= Path / Filename + TEXT("-edges.csv");

		TUniquePtr<FArchive> Ar(Ifm.CreateFileWriter(*EdgesFilename));
		if (!Ar.IsValid())
		{
			return FIoStatus(EIoErrorCode::WriteError);
		}

		UE_LOGF(LogIoStoreOnDemand, Display, "Saving '%ls'", *EdgesFilename);
		Ar->Logf(TEXT("Source, Target, Type, Weight, Id"));
		for (int32 Idx = 0; Idx < Graph.GetEdgeCount(); ++Idx)
		{
			const FPackageGraph::FEdgeData&	Edge	= Graph.GetEdgeData(FPackageGraph::FEdge { .Index = Idx });
			const float						Weight	= 1.0f;
			const TCHAR*					Type	= TEXT("Directed");
			Ar->Logf(TEXT("%d, %d, %s, %.2f, %d"), Edge.From.Index, Edge.To.Index, Type, Weight, Idx);
		}

		if (!Ar->Close())
		{
			return FIoStatus(EIoErrorCode::WriteError);
		}
	}

	return FIoStatus::Ok;
}

////////////////////////////////////////////////////////////////////////////////
void FPackageGraph::Reserve(int32 ContainerCount, int32 PackageCount)
{
	ContainerNodeData.Reserve(ContainerCount);
	NodeData.Reserve(PackageCount);
	EdgeData.Reserve(PackageCount);
	PackageIdToNode.Reserve(PackageCount);
}

FPackageGraph::FContainerNode FPackageGraph::AddContainerNode(FString&& Name, const FIoContainerId& Id)
{
	const int32 Index = ContainerNodeData.Num();
	ContainerNodeData.Add(FContainerData
	{
		.Name	= MoveTemp(Name),
		.Id		= Id
	});

	return FContainerNode { .Index = Index };
}

FPackageGraph::FNode FPackageGraph::AddNode(
	FPackageGraph::FContainerNode ContainerNode,
	const FPackageId& PackageId,
	const FString* Filename,
	uint64 ChunkSize)
{
	check(ContainerNode.IsValid());
	if (FNode Node = PackageIdToNode.FindRef(PackageId); Node.IsValid())
	{
		return Node;
	}

	const int32 Index	= NodeData.Num();
	FNodeData& Data		= NodeData.AddDefaulted_GetRef();
	Data.PackageId		= PackageId;
	Data.Size			= ChunkSize;
	Data.ContainerNode	= ContainerNode;

	if (Filename != nullptr)
	{
		Data.Filename	= *Filename;
	}

	FNode NewNode{Index};
	PackageIdToNode.Add(PackageId, NewNode);
	ContainerNodeData[ContainerNode.Index].PackageNodes.Add(NewNode);

	return NewNode;
}

void FPackageGraph::AddEdge(FNode From, FNode To, bool bSoft)
{
	check(From.IsValid() && To.IsValid());

	const FEdge Edge { .Index = EdgeData.Num() };
	EdgeData.Add(FEdgeData { .From = From, .To = To, .bSoft = bSoft });

	FNodeData& FromNodeData = NodeData[From.Index];
	FNodeData& ToNodeData	= NodeData[To.Index];

	FromNodeData.Connections.Out.Add(Edge);
	ToNodeData.Connections.In.Add(Edge);
}

void FPackageGraph::AddMissingEdge(FNode From, const FPackageId To, bool bSoft)
{
	check(From.IsValid()); 

	const FMissingEdge MissingEdge { .Index = MissingEdgeData.Num() };
	MissingEdgeData.Add(FMissingEdgeData { .From = From, .To = To, .bSoft = bSoft});

	FNodeData& FromNodeData = NodeData[From.Index];
}

void FPackageGraph::AddRedirect(const FPackageId& From, const FPackageId& To)
{
	Redirects.Add(FRedirect
	{
		.From	= From,
		.To		= To
	});
}

void FPackageGraph::AddShaderMapHashes(FNode Node, TConstArrayView<FShaderHash> Hashes)
{
	if (Hashes.IsEmpty())
	{
		return;
	}

	FNodeData& Data			= NodeData[Node.Index];
	Data.ShaderMapOffset	= ShaderMapHashes.Num();
	Data.ShaderMapCount		= Hashes.Num();

	ShaderMapHashes.Append(Hashes);

	for (const FShaderHash& Hash : Hashes)
	{
		ShaderHashToNode.Add(Hash, Node);
	}
}

TConstArrayView<FShaderHash> FPackageGraph::GetShaderMapHashes(const FNodeData& Data)
{
	if (Data.ShaderMapCount < 1)
	{
		return TConstArrayView<FShaderHash>();
	}

	return MakeArrayView(ShaderMapHashes.GetData() + Data.ShaderMapOffset, Data.ShaderMapCount);
}

void FPackageGraph::GetNodesForShaderMapHash(const FShaderHash& Hash, TArray<FNode>& Out)
{
	ShaderHashToNode.MultiFind(Hash, Out);
}

void FPackageGraph::Traverse(FNode StartNode, bool bIncoming, bool bSkipSoftRefs, FVisitor& Visitor)
{
	TSet<FNode>		Visited;
	TArray<FNode>	Stack;

	check(StartNode.IsValid());

	Stack.Push(StartNode);
	while (Stack.IsEmpty() == false)
	{
		const FNode Node = Stack.Pop();

		bool bAlreadyInSet = false;
		Visited.Add(Node, &bAlreadyInSet);
		if (bAlreadyInSet)
		{
			continue;
		}

		const FNodeData& Data = GetNodeData(Node);
		if (Visitor.Visit(*this, Node, Data) == false)
		{
			return;
		}

		const TSet<FEdge>& Edges = bIncoming ? Data.Connections.In : Data.Connections.Out;
		for (FEdge Edge : Edges)
		{
			const FEdgeData& ToTraverseEdgeData = EdgeData[Edge.Index];
			if (ToTraverseEdgeData.bSoft && bSkipSoftRefs)
			{
				continue;
			}

			const FNode ToTraverse = bIncoming ? ToTraverseEdgeData.From : ToTraverseEdgeData.To;
			if (Visited.Contains(ToTraverse) == false)
			{
				Stack.Push(ToTraverse);
			}
		}
	}
}

void FPackageGraph::Build(
	TConstArrayView<FString> ContainerFilenames,
	const FKeyChain& KeyChain,
	bool bIncludeSoftRefs,
	bool bShaderMaps,
	bool bIncludeOnDemand,
	FPackageGraph& Graph)
{
	using namespace UE::IoStore::Serialization;
	using namespace UE::IoStore::Serialization::V1;

	UE_LOGF(LogIoStoreOnDemand, Display, "Building package graph...");

	TMap<FGuid, FAES::FAESKey> EncryptionKeys;
	for (const TPair<FGuid, FNamedAESKey>& KeyPair: KeyChain.GetEncryptionKeys())
	{
		EncryptionKeys.Add(KeyPair.Key, KeyPair.Value.Key);
	}

	struct FLoadedContainer
	{
		FString						Name;
		FIoStoreReader				Reader;
		FOnDemandTocStorage			OnDemandContainerStorage;
		FOnDemandContainerTocView	OnDemandContainerView;
		FIoContainerHeader			Header;
		TMap<FIoChunkId, FString>	ChunkFileNamesMap;
		bool						bOnDemand = false;
	};
	TArray<TUniquePtr<FLoadedContainer>> Containers;
	Containers.Reserve(ContainerFilenames.Num());
	int32 TotalPackages = 0;

	// Load all containers
	for (const FString& Filename : ContainerFilenames)
	{
		// Only .utoc file(s) 
		if (FPaths::GetExtension(Filename, false) != TEXT("utoc") && FPaths::GetExtension(Filename, false) != TEXT("uondemandtoc"))
		{
			continue;
		}

		// Skip .o.utoc
		if (FPaths::GetExtension(FPaths::GetBaseFilename(Filename), false) == TEXT("o"))
		{
			continue;
		}

		FString Path = Filename;

		if (FPaths::GetExtension(Filename, false) == TEXT("uondemandtoc"))
		{
			if (bIncludeOnDemand == false)
			{
				UE_LOGF(LogIoStoreOnDemand, Display, "Skipping '%ls'", *Filename);
				continue;
			}

			TIoStatusOr<FOnDemandTocReader> MaybeReader = FOnDemandTocReader::Read(Filename);

			if (MaybeReader.IsOk() == false)
			{
				UE_LOGF(LogIoStoreOnDemand, Error, "Failed to open container '%ls' for reading due to: '%ls'",
					*Path, *MaybeReader.Status().ToString());
				continue;
			}

			FOnDemandTocReader Reader = MaybeReader.ConsumeValueOrDie();

			for (const FOnDemandContainerEntry& ContainerEntry : Reader.Containers())
			{
				TUniquePtr<FLoadedContainer> Container = MakeUnique<FLoadedContainer>();
				Container->bOnDemand = true;

				TIoStatusOr<FOnDemandContainerTocView> MaybeContainerView =
					Reader.ReadContainer(ContainerEntry, Container->OnDemandContainerStorage, EOnDemandTocReaderOptions::None);

				if (MaybeContainerView.IsOk() == false)
				{
					UE_LOGF(LogIoStoreOnDemand, Error, "Failed to read on-demand container TOC '%ls', reason: %ls",
						*Filename, *MaybeContainerView.Status().ToString());
					continue;
				}

				FOnDemandContainerTocView ContainerView = MaybeContainerView.ConsumeValueOrDie();
				if (ContainerView.ContainerHeaderChunk.GetSize() > 0)
				{
					FLargeMemoryReader Ar(ContainerView.ContainerHeaderChunk.GetData(), ContainerView.ContainerHeaderChunk.GetSize());

					Ar << Container->Header;
					if (Ar.IsError())
					{
						UE_LOGF(LogIoStoreOnDemand, Error, "Failed to serialize container header, filename '%ls'", *Filename);
						continue;
					}

					if (Container->Header.SoftPackageReferencesSerialInfo.Size > 0)
					{
						Ar.Seek(Container->Header.SoftPackageReferencesSerialInfo.Offset);
						Ar << Container->Header.SoftPackageReferences;
					}

					if (Ar.IsError())
					{
						UE_LOGF(LogIoStoreOnDemand, Error, "Failed to serialize container soft references, filename '%ls'", *Filename);
						continue;
					}

					for (const FIoContainerHeaderPackageRedirect& Redirect : Container->Header.PackageRedirects)
					{
						Graph.AddRedirect(Redirect.SourcePackageId, Redirect.TargetPackageId);
					}
				}

				Container->OnDemandContainerView = ContainerView;
				TotalPackages += Container->Header.PackageIds.Num();
				Container->Name = FString(ContainerView.Header.ContainerName());
				Containers.Add(MoveTemp(Container));
			}
		}
		else
		{
			TUniquePtr<FLoadedContainer> Container = MakeUnique<FLoadedContainer>();
			FIoStatus Status = Container->Reader.Initialize(*FPaths::ChangeExtension(Path, TEXT("")), EncryptionKeys);

			if (!Status.IsOk())
			{
				UE_LOGF(LogIoStoreOnDemand, Error, "Failed to open container '%ls' for reading due to: '%ls'", *Path, *Status.ToString());
				continue;
			}

			const FIoChunkId		HeaderChunkId = CreateContainerHeaderChunkId(Container->Reader.GetContainerId());
			TIoStatusOr<FIoBuffer>	Chunk = Container->Reader.Read(HeaderChunkId, FIoReadOptions());

			if (Chunk.IsOk() == false)
			{
				// Container header isn't required 
				continue;
			}

			FIoBuffer			Buffer = Chunk.ValueOrDie();
			FLargeMemoryReader	Ar(Buffer.GetData(), Buffer.GetSize());

			Ar << Container->Header;
			if (Ar.IsError())
			{
				UE_LOGF(LogIoStoreOnDemand, Error, "Failed to serialize container header, filename '%ls'", *Filename);
				continue;
			}

			if (Container->Header.SoftPackageReferencesSerialInfo.Size > 0)
			{
				Ar.Seek(Container->Header.SoftPackageReferencesSerialInfo.Offset);
				Ar << Container->Header.SoftPackageReferences;
			}

			if (Ar.IsError())
			{
				UE_LOGF(LogIoStoreOnDemand, Error, "Failed to serialize container soft references, filename '%ls'", *Filename);
				continue;
			}

			for (const FIoContainerHeaderPackageRedirect& Redirect : Container->Header.PackageRedirects)
			{
				Graph.AddRedirect(Redirect.SourcePackageId, Redirect.TargetPackageId); 
			}

			Container->Reader.GetDirectoryIndexReader().IterateDirectoryIndex(FIoDirectoryIndexHandle::RootDirectory(), TEXT(""),
				[&Container](FStringView Filename, uint32 TocEntryIndex) -> bool
				{
					TIoStatusOr<FIoStoreTocChunkInfo> ChunkInfo = Container->Reader.GetChunkInfo(TocEntryIndex);
					if (ChunkInfo.IsOk())
					{
						Container->ChunkFileNamesMap.Add(ChunkInfo.ValueOrDie().Id, FString(Filename));
					}
					return true;
				});

			TotalPackages += Container->Header.PackageIds.Num();
			Container->Name = FPaths::GetBaseFilename(Filename);
			Containers.Add(MoveTemp(Container));
		}
	}

	Graph.Reserve(Containers.Num(), TotalPackages);

	// Create graph nodes 
	for (const TUniquePtr<FLoadedContainer>& Container : Containers)
	{
		const FIoContainerHeader& Header = Container->Header;
		const FIoContainerId ContainerId = Container->bOnDemand
			? Container->OnDemandContainerView.Header.ContainerId()
			: Container->Reader.GetContainerId();

		FString								Name = Container->Name;
		const FPackageGraph::FContainerNode ContainerNode = Graph.AddContainerNode(MoveTemp(Name), ContainerId);
		const int32							NodeCount = Graph.GetNodeCount();

		for (int32 Idx = 0; Idx < Header.PackageIds.Num(); ++Idx)
		{
			const FPackageId	PackageId = Header.PackageIds[Idx];
			const FIoChunkId	ChunkId = CreatePackageDataChunkId(PackageId);
			const FString*		ChunkFilename = Container->ChunkFileNamesMap.Find(ChunkId);

			uint64 ChunkSize = 0;
			if (Container->bOnDemand)
			{
				const FOnDemandContainerTocView& ContainerView = Container->OnDemandContainerView;
				const int32 Index = Algo::LowerBound(ContainerView.ChunkIds, ChunkId);
				if (Index < 0 || Index >= ContainerView.ChunkIds.Num() || ContainerView.ChunkIds[Index] != ChunkId)
				{
					UE_LOGF(LogIoStoreOnDemand, Error, "Failed to find chunk info for package '%ls'", *LexToString(PackageId));
					continue;
				}
				ChunkSize = ContainerView.ChunkEntries[Index].GetDiskSize();
			}
			else
			{
				TIoStatusOr<FIoStoreTocChunkInfo> ChunkInfo = Container->Reader.GetChunkInfo(ChunkId);
				if (ChunkInfo.IsOk() == false)
				{
					UE_LOGF(LogIoStoreOnDemand, Error, "Failed to find chunk info for package '%ls'", *LexToString(PackageId));
					continue;
				}
				ChunkSize = Align(ChunkInfo.ValueOrDie().CompressedSize, 16);
			}

			Graph.AddNode(ContainerNode, PackageId, ChunkFilename, ChunkSize);
		}

		UE_LOGF(LogIoStoreOnDemand, Display, "Created %d node(s) from '%ls'", Graph.GetNodeCount() - NodeCount, *Container->Name);
	}

	// Create graph edges
	for (const TUniquePtr<FLoadedContainer>& Container : Containers)
	{
		const FString&				Name = Container->Name;
		const FIoContainerHeader&	Header = Container->Header;
		const int32					EdgeCount = Graph.GetEdgeCount();

		TConstArrayView<FFilePackageStoreEntry> PackageStoreEntries = MakeArrayView(
			reinterpret_cast<const FFilePackageStoreEntry*>(Header.StoreEntries.GetData()), Header.PackageIds.Num());

		TConstArrayView<FFilePackageStoreEntrySoftReferences> AllSoftReferences;
		if (Header.SoftPackageReferences.bContainsSoftPackageReferences)
		{
			AllSoftReferences = MakeArrayView<const FFilePackageStoreEntrySoftReferences>(
				reinterpret_cast<const FFilePackageStoreEntrySoftReferences*>(Header.SoftPackageReferences.PackageIndices.GetData()),
				Header.PackageIds.Num());
		}

		for (int32 Idx = 0; Idx < Header.PackageIds.Num(); ++Idx)
		{
			const FPackageId PackageId			= Header.PackageIds[Idx];
			const FFilePackageStoreEntry& Entry = PackageStoreEntries[Idx];

			// Hard reference(s)
			FPackageGraph::FNode FromNode = Graph.GetNode(PackageId);
			check(FromNode.IsValid());
			for (const FPackageId& ImportedPackageId : Entry.ImportedPackages)
			{
				const bool					bSoftEdge = false;
				const FPackageGraph::FNode	ToNode = Graph.GetNode(ImportedPackageId);
				if (ToNode.IsValid())
				{
					//TODO: Dedup edges
					Graph.AddEdge(FromNode, ToNode, bSoftEdge);
				}
				else
				{
					Graph.AddMissingEdge(FromNode, ImportedPackageId, bSoftEdge); 
				}
			}

			// Soft reference(s)
			if (bIncludeSoftRefs && AllSoftReferences.IsEmpty() == false)
			{
				const FFilePackageStoreEntrySoftReferences& SoftRefs = AllSoftReferences[Idx];
				if (SoftRefs.Indices.Num() > 0)
				{
					for (int32 SoftRefIdx : SoftRefs.Indices)
					{
						const bool					bSoftEdge = true;
						const FPackageId			SoftRef = Header.SoftPackageReferences.PackageIds[SoftRefIdx];
						const FPackageGraph::FNode	ToNode = Graph.GetNode(SoftRef);
						if (ToNode.IsValid())
						{
							Graph.AddEdge(FromNode, ToNode, bSoftEdge);
						}
						else
						{
							Graph.AddMissingEdge(FromNode, SoftRef, bSoftEdge);
						}
					}
				}
			}

			// ShaderMaps
			if (bShaderMaps)
			{
				Graph.AddShaderMapHashes(FromNode, MakeArrayView(Entry.ShaderMapHashes.Data(), Entry.ShaderMapHashes.Num()));
			}
		}

		UE_LOGF(LogIoStoreOnDemand, Display, "Created %d edges(s) from '%ls'", Graph.GetEdgeCount() - EdgeCount, *Name);
	}
}

FIoStatus FPackageGraph::Save(const FPackageGraph& Graph, const FString& Filename)
{
	const FString Ext = FPaths::GetExtension(Filename);

	if (Ext == TEXT("json") || Ext == TEXT("cb"))
	{
		UE_LOGF(LogIoStoreOnDemand, Display, "Saving graph to '%ls'", *Filename);

		IFileManager&	Ifm = IFileManager::Get();
		FCbWriter		Writer;

		Writer << Graph;

		TUniquePtr<FArchive> Ar = TUniquePtr<FArchive>(Ifm.CreateFileWriter(*Filename));
		if (Ar.IsValid())
		{
			if (Ext == TEXT("cb"))
			{
				Writer.Save(*Ar);
			}
			else
			{
				FCbObject GraphCb = Writer.Save().AsObject();
				TUtf8StringBuilder<4096> Sb;
				CompactBinaryToJson(GraphCb, Sb);
				UTF8CHAR UTF8BOM[] = { (UTF8CHAR)0xEF, (UTF8CHAR)0xBB, (UTF8CHAR)0xBF };
				Ar->Serialize(&UTF8BOM, UE_ARRAY_COUNT(UTF8BOM) * sizeof(UTF8CHAR));
				Ar->Serialize(Sb.GetData(), Sb.Len() * sizeof(UTF8CHAR));
			}
			Ar->Close();

			if (Ar->IsError())
			{
				return FIoStatus(EIoErrorCode::WriteError);
			}

			return FIoStatus::Ok;
		}
		else
		{
			return FIoStatus(EIoErrorCode::FileOpenFailed);
		}
	}
	else if (Ext == TEXT("csv"))
	{
		return ExportGephiCsv(Graph, Filename);
	}

	return FIoStatus(EIoErrorCode::InvalidParameter, TEXT("Unsupported file format"));
}

////////////////////////////////////////////////////////////////////////////////
static bool LoadFromCompactBinary(FCbFieldView Field, FPackageGraph::FIndex& Index)
{
	if (Field.IsInteger())
	{
		Index.Index = Field.AsInt32();
		return true;
	}

	return false;
}

////////////////////////////////////////////////////////////////////////////////
static FCbWriter& operator<<(FCbWriter& Writer, const FPackageGraph::FIndex& Index)
{
	Writer.AddInteger(Index.Index);
	return Writer;
}

////////////////////////////////////////////////////////////////////////////////
static bool LoadFromCompactBinary(FCbFieldView Field, FPackageGraph::FEdgeData& Edge)
{
	if (FCbObjectView Obj = Field.AsObjectView())
	{
		Edge.From	= FPackageGraph::FNode { .Index = Obj["From"].AsInt32() };
		Edge.To		= FPackageGraph::FNode { .Index = Obj["To"].AsInt32() };
		Edge.bSoft	= Obj["Soft"].AsBool();

		return true;
	}

	return false;
}

////////////////////////////////////////////////////////////////////////////////
static FCbWriter& operator<<(FCbWriter& Writer, const FPackageGraph::FEdgeData& Edge)
{
	Writer.BeginObject();
	Writer << UTF8TEXT("From") << Edge.From;
	Writer << UTF8TEXT("To") << Edge.To;
	Writer << UTF8TEXT("Soft") << Edge.bSoft;
	Writer.EndObject();

	return Writer;
}

////////////////////////////////////////////////////////////////////////////////
static bool LoadFromCompactBinary(FCbFieldView Field, FPackageGraph::FNodeData& Node)
{
	FCbObjectView Obj = Field.AsObjectView();

	if (!Obj)
	{
		return false;
	}

	Node.Filename			= FString(Obj["Filename"].AsString());
	Node.PackageId			= FPackageId::FromValue(Obj["PackageId"].AsUInt64());
	Node.Size				= Obj["Size"].AsUInt64();
	Node.ContainerNode		= FPackageGraph::FContainerNode { .Index = Obj["Container"].AsInt32() };
	Node.ShaderMapOffset	= Obj["ShaderMapOffset"].AsInt32(INDEX_NONE);
	Node.ShaderMapCount		= Obj["ShaderMapCount"].AsInt32(0);

	if (FCbArrayView ArrayView = Obj["In"].AsArrayView())
	{
		Node.Connections.In.Reserve(int32(ArrayView.Num()));
		for (FCbFieldView ArrayField : ArrayView)
		{
			Node.Connections.In.Add(FPackageGraph::FEdge { .Index = ArrayField.AsInt32() });
		}
	}

	if (FCbArrayView ArrayView = Obj["Out"].AsArrayView())
	{
		Node.Connections.Out.Reserve(int32(ArrayView.Num()));
		for (FCbFieldView ArrayField : ArrayView)
		{
			Node.Connections.Out.Add(FPackageGraph::FEdge { .Index = ArrayField.AsInt32() });
		}
	}

	return true;
}

////////////////////////////////////////////////////////////////////////////////
static FCbWriter& operator<<(FCbWriter& Writer, const FPackageGraph::FNodeData& Data)
{
	Writer.BeginObject();
	Writer << UTF8TEXT("Filename") << Data.Filename;
	Writer << UTF8TEXT("PackageId") << Data.PackageId.Value();
	Writer << UTF8TEXT("Size") << Data.Size;
	Writer << UTF8TEXT("Container") << Data.ContainerNode;
	Writer << UTF8TEXT("ShaderMapOffset") << Data.ShaderMapOffset;
	Writer << UTF8TEXT("ShaderMapCount") << Data.ShaderMapCount;

	Writer.BeginArray(UTF8TEXT("In"));
	for (FPackageGraph::FEdge Edge : Data.Connections.In)
	{
		Writer << Edge;
	}
	Writer.EndArray();

	Writer.BeginArray(UTF8TEXT("Out"));
	for (FPackageGraph::FEdge Edge : Data.Connections.Out)
	{
		Writer << Edge;
	}
	Writer.EndArray();

	Writer.EndObject();
	return Writer;
}

////////////////////////////////////////////////////////////////////////////////
static bool LoadFromCompactBinary(FCbFieldView Field, FPackageGraph::FContainerData& Container)
{
	FCbObjectView Obj = Field.AsObjectView();

	if (!Obj)
	{
		return false;
	}

	Container.Name = FString(Obj["Name"].AsString());
	if (!LoadFromCompactBinary(Obj["Id"], Container.Id))
	{
		return false;
	}

	if (FCbArrayView ArrayView = Obj["In"].AsArrayView())
	{
		Container.Connections.In.Reserve(int32(ArrayView.Num()));
		for (FCbFieldView ArrayField : ArrayView)
		{
			Container.Connections.In.Add(FPackageGraph::FEdge { .Index = ArrayField.AsInt32() });
		}
	}

	if (FCbArrayView ArrayView = Obj["Out"].AsArrayView())
	{
		Container.Connections.Out.Reserve(int32(ArrayView.Num()));
		for (FCbFieldView ArrayField : ArrayView)
		{
			Container.Connections.Out.Add(FPackageGraph::FEdge { .Index = ArrayField.AsInt32() });
		}
	}

	return true;
}

////////////////////////////////////////////////////////////////////////////////
static FCbWriter& operator<<(FCbWriter& Writer, const FPackageGraph::FContainerData& Container)
{
	Writer.BeginObject();
	Writer << UTF8TEXT("Name") << Container.Name;
	Writer << UTF8TEXT("Id") << Container.Id;

	Writer.BeginArray(UTF8TEXT("In"));
	for (FPackageGraph::FEdge Edge : Container.Connections.In)
	{
		Writer << Edge;
	}
	Writer.EndArray();

	Writer.BeginArray(UTF8TEXT("Out"));
	for (FPackageGraph::FEdge Edge : Container.Connections.Out)
	{
		Writer << Edge;
	}
	Writer.EndArray();

	Writer.EndObject();
	return Writer;
}

////////////////////////////////////////////////////////////////////////////////
static bool LoadFromCompactBinary(FCbFieldView Field, FPackageGraph::FMissingEdgeData& MissingEdge)
{
	if (FCbObjectView Obj = Field.AsObjectView())
	{
		MissingEdge.From	= FPackageGraph::FNode { .Index = Obj["From"].AsInt32() };
		MissingEdge.To		= FPackageId::FromValue(Obj["To"].AsUInt64());
		MissingEdge.bSoft	= Obj["Soft"].AsBool();

		return true;
	}

	return false; 
}

////////////////////////////////////////////////////////////////////////////////
static FCbWriter& operator<<(FCbWriter& Writer, const FPackageGraph::FMissingEdgeData& MissingEdge)
{
	Writer.BeginObject();
	Writer << UTF8TEXT("From") << MissingEdge.From;
	Writer << UTF8TEXT("To") << MissingEdge.To.Value();
	Writer << UTF8TEXT("Soft") << MissingEdge.bSoft;
	Writer.EndObject();

	return Writer;
}

////////////////////////////////////////////////////////////////////////////////
static bool LoadFromCompactBinary(FCbFieldView Field, FPackageGraph::FRedirect& Redirect)
{
	if (FCbObjectView Obj = Field.AsObjectView())
	{
		Redirect.From	= FPackageId::FromValue(Obj["From"].AsUInt64());
		Redirect.To		= FPackageId::FromValue(Obj["To"].AsUInt64());

		return true;
	}

	return false;
}

////////////////////////////////////////////////////////////////////////////////
static FCbWriter& operator<<(FCbWriter& Writer, const FPackageGraph::FRedirect& Redirect)
{
	Writer.BeginObject();
	Writer << UTF8TEXT("From") << Redirect.From.Value();
	Writer << UTF8TEXT("To") << Redirect.To.Value();
	Writer.EndObject();

	return Writer;
}

////////////////////////////////////////////////////////////////////////////////
FCbWriter& operator<<(FCbWriter& Writer, const FPackageGraph& Graph)
{
	Writer.BeginObject();

	if (Graph.ContainerNodeData.IsEmpty() == false)
	{
		Writer.BeginArray(UTF8TEXT("Containers"));
		for (const FPackageGraph::FContainerData& Data : Graph.ContainerNodeData)
		{
			Writer << Data;
		}
		Writer.EndArray();
	}

	if (Graph.NodeData.IsEmpty() == false)
	{
		Writer.BeginArray(UTF8TEXT("Nodes"));
		for (const FPackageGraph::FNodeData& Data : Graph.NodeData)
		{
			Writer << Data;
		}
		Writer.EndArray();
	}

	if (Graph.EdgeData.IsEmpty() == false)
	{
		Writer.BeginArray(UTF8TEXT("Edges"));
		for (const FPackageGraph::FEdgeData& Edge : Graph.EdgeData)
		{
			Writer << Edge;
		}
		Writer.EndArray();
	}

	if (Graph.MissingEdgeData.IsEmpty() == false)
	{
		Writer.BeginArray(UTF8TEXT("MissingEdges"));
		for (const FPackageGraph::FMissingEdgeData& Missing : Graph.MissingEdgeData)
		{
			Writer << Missing;
		}
		Writer.EndArray();
	}

	if (Graph.Redirects.IsEmpty() == false)
	{
		Writer.BeginArray(UTF8TEXT("Redirects"));
		for (const FPackageGraph::FRedirect& Redirect : Graph.Redirects)
		{
			Writer << Redirect;
		}
		Writer.EndArray();
	}

	if (Graph.ShaderMapHashes.IsEmpty() == false)
	{
		Writer.BeginArray(UTF8TEXT("ShaderMapHashes"));
		for (const FShaderHash& Hash : Graph.ShaderMapHashes)
		{
			Writer << Hash;
		}
		Writer.EndArray();
	}

	Writer.EndObject();

	return Writer;
}

////////////////////////////////////////////////////////////////////////////////
bool LoadFromCompactBinary(FCbFieldView Field, FPackageGraph& Graph)
{
	FCbObjectView Obj = Field.AsObjectView();

	if (!Obj)
	{
		return false;
	}

	if (FCbArrayView ArrayView = Obj["Containers"].AsArrayView())
	{
		Graph.ContainerNodeData.Reserve(int32(ArrayView.Num()));
		for (FCbFieldView ArrayField : ArrayView)
		{
			if (!LoadFromCompactBinary(ArrayField, Graph.ContainerNodeData.AddDefaulted_GetRef()))
			{
				return false;
			}
		}
	}

	if (FCbArrayView ArrayView = Obj["Nodes"].AsArrayView())
	{
		Graph.NodeData.Reserve(int32(ArrayView.Num()));
		for (FCbFieldView ArrayField : ArrayView)
		{
			const int32					Index = Graph.NodeData.Num();
			FPackageGraph::FNode		Node { .Index = Index  };
			FPackageGraph::FNodeData&	Data = Graph.NodeData.AddDefaulted_GetRef();

			if (LoadFromCompactBinary(ArrayField, Data))
			{
				Graph.PackageIdToNode.Add(Data.PackageId, Node);
				Graph.ContainerNodeData[Data.ContainerNode.Index].PackageNodes.Add(Node);
			}
			else
			{
				return false;
			}
		}
	}

	if (FCbArrayView ArrayView = Obj["Edges"].AsArrayView())
	{
		Graph.EdgeData.Reserve(int32(ArrayView.Num()));
		for (FCbFieldView ArrayField : ArrayView)
		{
			if (!LoadFromCompactBinary(ArrayField, Graph.EdgeData.AddDefaulted_GetRef()))
			{
				return false;
			}
		}
	}

	if (FCbArrayView ArrayView = Obj["MissingEdges"].AsArrayView())
	{
		Graph.MissingEdgeData.Reserve(int32(ArrayView.Num()));
		for (FCbFieldView ArrayField : ArrayView)
		{
			if (!LoadFromCompactBinary(ArrayField, Graph.MissingEdgeData.AddDefaulted_GetRef()))
			{
				return false;
			}
		}
	}

	if (FCbArrayView ArrayView = Obj["Redirects"].AsArrayView())
	{
		Graph.Redirects.Reserve(int32(ArrayView.Num()));
		for (FCbFieldView ArrayField : ArrayView)
		{
			FPackageGraph::FRedirect Redirect;
			if (LoadFromCompactBinary(ArrayField, Redirect))
			{
				Graph.Redirects.Add(Redirect);
			}
			else
			{
				return false;
			}
		}
	}

	if (FCbArrayView ArrayView = Obj["ShaderMapHashes"].AsArrayView())
	{
		Graph.ShaderMapHashes.Reserve(int32(ArrayView.Num()));
		for (FCbFieldView ArrayField : ArrayView)
		{
			FShaderHash Hash;
			if (LoadFromCompactBinary(ArrayField, Hash))
			{
				Graph.ShaderMapHashes.Add(Hash);
			}
			else
			{
				UE_LOGF(LogIoStoreOnDemand, Error, "Failed to load shader map hash");
				return false;
			}
		}
	}

	for (int32 Index = 0; const FPackageGraph::FNodeData& Data : Graph.NodeData)
	{
		const FPackageGraph::FNode Node { .Index = Index++ };
		TConstArrayView<FShaderHash> Hashes = Graph.GetShaderMapHashes(Data);
		for (const FShaderHash& Hash : Hashes)
		{
			Graph.ShaderHashToNode.Add(Hash,  Node);
		}
	}

	return true;
}

////////////////////////////////////////////////////////////////////////////////
FIoStatus LoadFromCompactBinary(const FString& Filename, FPackageGraph& OutGraph)
{
	IFileManager& Ifm = IFileManager::Get();

	TUniquePtr<FArchive> Ar(Ifm.CreateFileReader(*Filename));

	if (Ar.IsValid() == false)
	{
		return FIoStatusBuilder(EIoErrorCode::FileNotOpen) << TEXT("Failed to open '") << Filename << TEXT("'");
	}

	if (FCbField GraphCb = LoadCompactBinary(*Ar))
	{
		if (LoadFromCompactBinary(GraphCb, OutGraph))
		{
			return FIoStatus::Ok;
		}
	}

	return FIoStatusBuilder(EIoErrorCode::ReadError) << TEXT("Failed to read combact binary from '") << Filename << TEXT("'");
}

////////////////////////////////////////////////////////////////////////////////
namespace GraphTool
{

////////////////////////////////////////////////////////////////////////////////
static TArray<FString> GlobContainers(const FContext& Context)
{
	IFileManager& Ifm = IFileManager::Get();

	const FString GlobPattern = FString(Context.Get<FStringView>(TEXT("ContainerGlob")));
	TArray<FString> Ret;

	if (Ifm.FileExists(*GlobPattern))
	{
		Ret.Add(GlobPattern);
	}
	else if (Ifm.DirectoryExists(*GlobPattern))
	{
		FString Directory = GlobPattern;
		FPaths::NormalizeDirectoryName(Directory);

		TArray<FString> FoundContainerFiles;
		Ifm.FindFiles(FoundContainerFiles, *(Directory / TEXT("*.utoc")), true, false);

		for (const FString& Filename : FoundContainerFiles)
		{
			Ret.Emplace(Directory / Filename);
		}
	}
	else
	{
		FString Directory = FPaths::GetPath(GlobPattern);
		FPaths::NormalizeDirectoryName(Directory);

		TArray<FString> FoundContainerFiles;
		Ifm.FindFiles(FoundContainerFiles, *GlobPattern, true, false);

		for (const FString& Filename : FoundContainerFiles)
		{
			Ret.Emplace(Directory / Filename);
		}
	}

	return Ret;
}

} // UE::IoStore::Tool::GraphTool

////////////////////////////////////////////////////////////////////////////////
static int32 BuildGraphCommandEntry(const FContext& Context)
{
	GPrintLogCategory = false;
	GPrintLogVerbosity = false;
	GPrintLogTimes = ELogTimes::None;

	TArray<FString> Containers	= GraphTool::GlobContainers(Context);
	FKeyChain		KeyChain	= Common::LoadCryptoKeys(Context);
	FString			OutFilename	= FString(Context.Get<FStringView>(TEXT("-Out")));

	if (Containers.IsEmpty())
	{
		UE_LOGF(LogIoStoreOnDemand, Error, "Failed to find any container(s).");
		return -1;
	}

	if (OutFilename.IsEmpty())
	{
		UE_LOGF(LogIoStoreOnDemand, Error, "No filename specified");
		return -1;
	}

	const bool bSkipSoftRefs	= Context.Get<bool>(TEXT("-SkipSoftRefs"));
	const bool bShaderMaps		= Context.Get<bool>(TEXT("-ShaderMaps"));
	const bool bIncludeOnDemand = Context.Get<bool>(TEXT("-IncludeOnDemand"));

	FPackageGraph Graph;
	FPackageGraph::Build(Containers, KeyChain, !bSkipSoftRefs, bShaderMaps, bIncludeOnDemand, Graph);
	FIoStatus Status = FPackageGraph::Save(Graph, OutFilename);

	if (Status.IsOk() == false)
	{
		UE_LOGF(LogIoStoreOnDemand, Error, "%ls", *Status.ToString());
		return -1;
	}

	return 0; 
}

////////////////////////////////////////////////////////////////////////////////
static int32 QueryGraphCommandEntry(const FContext& Context)
{
	GPrintLogCategory = false;
	GPrintLogVerbosity = false;
	GPrintLogTimes = ELogTimes::None;

	FString			GraphFilename = FString(Context.Get<FStringView>(TEXT("Graph")));
	FPackageGraph	Graph;

	bool				bTraverse		= false;
	bool				bPrintDeps		= false;
	FPackageId			PackageId		= FPackageId();
	TArray<FShaderHash>	ShaderMapHashes;

	if (FStringView ShaderHashView = Context.Get<FStringView>(TEXT("-Shader")); ShaderHashView.IsEmpty() == false)
	{
		IFileManager& Ifm = IFileManager::Get();
		FString Filename = FString(ShaderHashView);

		auto ParseHash = [&ShaderMapHashes](const FString& Str)
		{
			if (Str.Len() == FShaderHash::GetStringLen())
			{
				LexFromString(ShaderMapHashes.AddDefaulted_GetRef(), *Str);
			}
			else
			{
				UE_LOGF(LogIoStoreOnDemand, Warning, "Skipping invalid shader hash '%ls' (expected %d hex chars)", *Str, FShaderHash::GetStringLen());
			}
		};

		if (Ifm.FileExists(*Filename))
		{
			TArray<FString> Lines;
			FFileHelper::LoadFileToStringArray(Lines, *Filename);
			for (const FString& Line : Lines)
			{
				if (Line.IsEmpty() == false)
				{
					ParseHash(Line);
				}
			}
		}
		else
		{
			ParseHash(FString(ShaderHashView));
		}
	}

	if (FStringView PackageNameOrId = Context.Get<FStringView>(TEXT("-Traverse")); PackageNameOrId.IsEmpty() == false)
	{
		LexFromString(PackageId, PackageNameOrId);
		bTraverse = true;
	}

	if (FStringView PackageNameOrId = Context.Get<FStringView>(TEXT("-Deps")); PackageNameOrId.IsEmpty() == false)
	{
		LexFromString(PackageId, PackageNameOrId);
		bPrintDeps	= true;
	}

	UE_LOGF(LogIoStoreOnDemand, Display, "Loading graph '%ls'", *GraphFilename);
	if (FIoStatus Status = LoadFromCompactBinary(GraphFilename, Graph); !Status.IsOk()) 
	{
		UE_LOGF(LogIoStoreOnDemand, Error, "%ls", *Status.ToString());
		return -1;
	}

	if (ShaderMapHashes.IsEmpty() == false)
	{
		UE_LOGF(LogIoStoreOnDemand, Display, "");
		UE_LOGF(LogIoStoreOnDemand, Display, "%-40s %-40s %-20s %s",
			"ShaderMapHash", "Container", "Package ID", "Filename");
		UE_LOGF(LogIoStoreOnDemand, Display, "----------------------------------------------------------------------------------------------------------------");
		for (const FShaderHash& Hash : ShaderMapHashes)
		{
			TArray<FPackageGraph::FNode> Nodes;
			Graph.GetNodesForShaderMapHash(Hash, Nodes);

			if (Nodes.IsEmpty())
			{
				UE_LOGF(LogIoStoreOnDemand, Display, "%-40ls %ls", *LexToString(Hash), TEXT("<NotFound>"));
				continue;
			}

			for (int32 Idx = 0; const FPackageGraph::FNode& Node : Nodes)
			{
				const FPackageGraph::FNodeData&			Data = Graph.GetNodeData(Node);
				const FPackageGraph::FContainerData&	ContainerData = Graph.GetContainerData(Data.ContainerNode);

				UE_LOGF(LogIoStoreOnDemand, Display, "%-40ls %-40ls %-20ls %ls",
					*LexToString(Hash), *ContainerData.Name, *LexToString(Data.PackageId), *Data.Filename);
			}
		}

		return 0;
	}

	if (PackageId.IsValid() == false)
	{
		UE_LOGF(LogIoStoreOnDemand, Error, "Invalid package ID 0x%ls", *LexToString(PackageId));
		return -1;
	}

	const FPackageGraph::FNode Node = Graph.GetNode(PackageId);
	if (Node.IsValid() == false)
	{
		UE_LOGF(LogIoStoreOnDemand, Error, "Failed to find graph node for 0x%ls", *LexToString(PackageId));
		return -1;
	}

	if (bTraverse)
	{
		struct FContainerStats
		{
			uint64 PackageCount = 0;
			uint64 TotalSize	= 0;
		};

		struct FVisitor final
			: public FPackageGraph::FVisitor
		{
			virtual bool Visit(const FPackageGraph& Graph, FPackageGraph::FNode Node, const FPackageGraph::FNodeData& Data) override
			{
				bool bAlreadyInSet = false;
				Visited.Add(Node, &bAlreadyInSet);
				check(!bAlreadyInSet);
				TraverseOrder.Add(Node);

				FContainerStats& Stats = ContainerStats.FindOrAdd(Data.ContainerNode);
				Stats.TotalSize += Data.Size;
				Stats.PackageCount++;
				TotalSize += Data.Size;

				return true;
			}

			TSet<FPackageGraph::FNode>								Visited;
			TArray<FPackageGraph::FNode>							TraverseOrder;
			TMap<FPackageGraph::FContainerNode, FContainerStats>	ContainerStats;
			uint64 TotalSize = 0;
		}; 

		const bool bSkipSoftRefs	= Context.Get<bool>(TEXT("-SkipSoftRefs"));
		const bool bIncoming		= Context.Get<bool>(TEXT("-Incoming"));

		UE_LOGF(LogIoStoreOnDemand, Display, "Traversing %s graph node(s) %s for package ID 0x%ls",
			bIncoming ? "incoming" : "outgoing",
			bSkipSoftRefs ? "skipping soft references" : "including soft references",
			*LexToString(PackageId));
		UE_LOGF(LogIoStoreOnDemand, Display, "");
		UE_LOGF(LogIoStoreOnDemand, Display, "%4s %-20s %-40s %-20s %s",
			"#", "Container ID", "Container", "Package ID", "Filename");
		UE_LOGF(LogIoStoreOnDemand, Display, "----------------------------------------------------------------------------------------------------");

		FVisitor Visitor;
		Graph.Traverse(Node, bIncoming, bSkipSoftRefs, Visitor);

		for (int32 Idx = 1; FPackageGraph::FNode Traversed : Visitor.TraverseOrder)
		{
			const FPackageGraph::FNodeData&			Data = Graph.GetNodeData(Traversed);
			const FPackageGraph::FContainerData&	ContainerData = Graph.GetContainerData(Data.ContainerNode);

			UE_LOGF(LogIoStoreOnDemand, Display, "%3d: 0x%-18ls %-40ls %s%-18ls '%ls' (%.2fKiB)",
				Idx++, *LexToString(ContainerData.Id), *ContainerData.Name, "0x", *LexToString(Data.PackageId), *Data.Filename, float(Data.Size) / 1024.0f);
		}

		UE_LOGF(LogIoStoreOnDemand, Display, "----------------------------------------------------------------------------------------------------");
		UE_LOGF(LogIoStoreOnDemand, Display, "Traversed %d package(s) of total %.2f KiB in %d container(s) for package ID 0x%ls",
				Visitor.TraverseOrder.Num(), float(Visitor.TotalSize) / 1024.0f, Visitor.ContainerStats.Num(), *LexToString(PackageId));

		for (const TPair<FPackageGraph::FContainerNode, FContainerStats>& Kv : Visitor.ContainerStats)
		{
			const FPackageGraph::FContainerData& ContainerData = Graph.GetContainerData(Kv.Key);
			UE_LOGF(LogIoStoreOnDemand, Display, "\t* %-40ls : %.2f KiB (%llu)", *ContainerData.Name, float(Kv.Value.TotalSize) / 1024.0f, Kv.Value.PackageCount);
		}
	}
	else if (bPrintDeps)
	{
		const bool								bIncoming		= Context.Get<bool>(TEXT("-Incoming"));
		const FPackageGraph::FNodeData&			NodeData		= Graph.GetNodeData(Node);
		const FPackageGraph::FContainerData&	ContainerData	= Graph.GetContainerData(NodeData.ContainerNode);

		UE_LOGF(LogIoStoreOnDemand, Display, "");
		UE_LOGF(LogIoStoreOnDemand, Display, "Dependencies: 0x%ls", *LexToString(NodeData.PackageId));
		UE_LOGF(LogIoStoreOnDemand, Display, "--------------------------------------------------------------------------------");
		UE_LOGF(LogIoStoreOnDemand, Display, "%-16s: %ls", "Container", *ContainerData.Name);
		UE_LOGF(LogIoStoreOnDemand, Display, "%-16s: '%ls'", "Filename", *LexToString(NodeData.Filename));
		UE_LOGF(LogIoStoreOnDemand, Display, "%-16s: %.2fKiB", "Size", float(NodeData.Size) / 1024.0f);
		UE_LOGF(LogIoStoreOnDemand, Display, "");

		UE_LOGF(LogIoStoreOnDemand, Display, "Outgoing:");
		UE_LOGF(LogIoStoreOnDemand, Display, "---------");
		for (int32 Idx = 1; FPackageGraph::FEdge Edge : NodeData.Connections.Out)
		{
			const FPackageGraph::FEdgeData&		EdgeData		= Graph.GetEdgeData(Edge);
			const FPackageGraph::FNodeData&		ToData			= Graph.GetNodeData(EdgeData.To);
			const FPackageGraph::FContainerData ToContainerData = Graph.GetContainerData(ToData.ContainerNode);
			UE_LOGF(LogIoStoreOnDemand, Display, "%d: %-40ls %s%-18ls '%ls' (%.2f KiB)",
				Idx++, *ToContainerData.Name, "0x", *LexToString(ToData.PackageId), *ToData.Filename, float(ToData.Size) / 1024.0f);
			ensure(EdgeData.From == Node);
		}

		if (bIncoming)
		{
			UE_LOGF(LogIoStoreOnDemand, Display, "");
			UE_LOGF(LogIoStoreOnDemand, Display, "Incoming:");
			UE_LOGF(LogIoStoreOnDemand, Display, "---------");
			for (int32 Idx = 1; FPackageGraph::FEdge Edge : NodeData.Connections.In)
			{
				const FPackageGraph::FEdgeData&		EdgeData			= Graph.GetEdgeData(Edge);
				const FPackageGraph::FNodeData&		FromData			= Graph.GetNodeData(EdgeData.From);
				const FPackageGraph::FContainerData FromContainerData	= Graph.GetContainerData(FromData.ContainerNode);
				UE_LOGF(LogIoStoreOnDemand, Display, "%d: %-40ls %s%-18ls '%ls' (%.2f KiB)",
						Idx++, *FromContainerData.Name, "0x", *LexToString(FromData.PackageId), *FromData.Filename, float(FromData.Size) / 1024.0f);
				ensure(EdgeData.To == Node);
			}
		}
	}

	return 0; 
}

////////////////////////////////////////////////////////////////////////////////
static FCommand GraphCommand(
	BuildGraphCommandEntry,
	TEXT("BuildGraph"),
	TEXT(""),
	{
		TArgument<FStringView>(TEXT("ContainerGlob"),	TEXT("Path globbed to discover input containers")),
		TArgument<FStringView>(TEXT("-CryptoKeys"),		TEXT("JSON-format keyring for input containers")),
		TArgument<bool>(TEXT("-SkipSoftRefs"),			TEXT("Whether to skip traverisng soft package reference(s).")),
		TArgument<bool>(TEXT("-ShaderMaps"),			TEXT("Whether to include shader map hashes.")),
		TArgument<bool>(TEXT("-IncludeOnDemand"),		TEXT("Whether to include on-demand container(s).")),
		TArgument<FStringView>(TEXT("-Out"),			TEXT("Filename to save the graph (JSON|CompactBinary)."))
	}
);

////////////////////////////////////////////////////////////////////////////////
static FCommand QueryGraphCommand(
	QueryGraphCommandEntry,
	TEXT("QueryGraph"),
	TEXT(""),
	{
		TArgument<FStringView>(TEXT("Graph"),		TEXT("Filepath to the package graph stored in compact binary format.")),
		TArgument<FStringView>(TEXT("-Traverse"),	TEXT("Traverse the graph starting from the specified package ID.")),
		TArgument<FStringView>(TEXT("-Deps"),		TEXT("Print dependencies for the specified package ID.")),
		TArgument<FStringView>(TEXT("-Shader"),		TEXT("Print packages referencing the specified shader hash.")),
		TArgument<bool>(TEXT("-SkipSoftRefs"),		TEXT("Whether to skip traverisng soft package reference(s).")),
		TArgument<bool>(TEXT("-Incoming"),			TEXT("Whether to traverse incoming package nodes."))
	}
);

} // namespace UE::IoStore::Tool
