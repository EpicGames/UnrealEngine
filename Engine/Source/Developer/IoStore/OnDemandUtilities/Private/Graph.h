// Copyright Epic Games, Inc. All Rights Reserved.

#include "CoreTypes.h"
#include "Containers/Map.h"
#include "Containers/Set.h"
#include "Hash/ShaderHash.h"
#include "IO/IoChunkId.h"
#include "IO/IoContainerId.h"
#include "IO/IoStatus.h"
#include "IO/PackageId.h"

class FCbWriter;
class FCbFieldView;
struct FKeyChain;

namespace UE::IoStore::Tool
{

////////////////////////////////////////////////////////////////////////////////
class FPackageGraph
{
public:
	struct FIndex
	{
		bool			IsValid() const					{ return Index != INDEX_NONE; }
		friend uint32	GetTypeHash(FIndex Node)		{ return GetTypeHash(Node.Index); }
		explicit		operator bool() const			{ return IsValid(); }
		bool			operator==(FIndex Other) const	{ return Index == Other.Index; }

		int32 Index = INDEX_NONE;
	};

	using FNode				= FIndex;
	using FEdge				= FIndex;
	using FContainerNode	= FIndex;
	using FMissingEdge		= FIndex;

	struct FConnections
	{
		TSet<FEdge>	In;
		TSet<FEdge>	Out;
	};

	struct FNodeData
	{
		FConnections	Connections;
		FString			Filename;
		FPackageId		PackageId;
		uint64			Size;
		FContainerNode	ContainerNode;
		int32			ShaderMapOffset = INDEX_NONE;
		int32			ShaderMapCount = 0;
	};

	struct FEdgeData
	{
		FNode	From;
		FNode	To;
		bool	bSoft = false;
	};

	struct FMissingEdgeData
	{
		FNode		From;
		FPackageId	To;
		bool		bSoft = false;
	};

	struct FContainerData
	{
		FConnections	Connections;
		TSet<FNode>		PackageNodes; // Transient
		FString			Name;
		FIoContainerId	Id;
	};

	struct FRedirect
	{
		friend uint32	GetTypeHash(const FRedirect &Redirect)		{ return GetTypeHash(Redirect.From); }
		bool			operator==(const FRedirect &Other) const	{ return From == Other.From; }
		bool			operator<(const FRedirect &Other) const		{ return From < Other.From; }

		FPackageId From;
		FPackageId To;
	};

	struct FVisitor
	{
		virtual bool Visit(const FPackageGraph& Graph, FNode Node, const FNodeData& Data) = 0;
	};

	void						Reserve(int32 ContainerCount, int32 PackageCount);
	FContainerNode				AddContainerNode(FString&& Name, const FIoContainerId& Id);
	const FContainerData&		GetContainerData(FContainerNode Node) const { return ContainerNodeData[Node.Index]; }
	FNode						AddNode(FContainerNode ContainerNode, const FPackageId& PackageId, const FString* Filename, uint64 ChunkSize);
	FNode						GetNode(const FPackageId& PackageId) const { return PackageIdToNode.FindRef(PackageId); }
	const FNodeData&			GetNodeData(FNode Node) const { return NodeData[Node.Index]; }
	int32						GetNodeCount() const { return NodeData.Num(); }
	void						AddEdge(FNode From, FNode To, bool bSoft);
	const FEdgeData&			GetEdgeData(FEdge Edge) const { return EdgeData[Edge.Index]; }
	int32						GetEdgeCount() const { return EdgeData.Num(); }
	void						AddMissingEdge(FNode From, const FPackageId To, bool bSoft);
	void						AddRedirect(const FPackageId& From, const FPackageId& To);
	void						AddShaderMapHashes(FNode Node, TConstArrayView<FShaderHash> Hashes);
	TConstArrayView<FShaderHash> GetShaderMapHashes(const FNodeData& Data);
	void						GetNodesForShaderMapHash(const FShaderHash& Hash, TArray<FNode>& Out);

	void						Traverse(FNode Node, bool bIncoming, bool bSkipSoftRefs, FVisitor& Visitor);

	friend FCbWriter&			operator<<(FCbWriter& Writer, const FPackageGraph& Graph);
	friend bool					LoadFromCompactBinary(FCbFieldView Field, FPackageGraph& Graph);

	static void					Build(
									TConstArrayView<FString> ContainerFilenames,
									const FKeyChain& KeyChain,
									bool bSkipSoftRefs,
									bool bShaderMaps,
									bool bIncludeOnDemand,
									FPackageGraph& Graph);
	static FIoStatus			Save(const FPackageGraph& Graph, const FString& Filename);

private:
	using FShaderHashMap		= TMultiMap<FShaderHash, FNode>;

	TArray<FNodeData>			NodeData;
	TArray<FEdgeData>			EdgeData;
	TArray<FMissingEdgeData>	MissingEdgeData;
	TArray<FShaderHash>			ShaderMapHashes;
	TArray<FContainerData>		ContainerNodeData;
	TSet<FRedirect>				Redirects;
	TMap<FPackageId, FNode>		PackageIdToNode; // Transient
	FShaderHashMap				ShaderHashToNode; // Transient
};

////////////////////////////////////////////////////////////////////////////////
FIoStatus	LoadFromCompactBinary(const FString& Filename, FPackageGraph& OutGraph);

} // namespace UE::IoStore::Tool
