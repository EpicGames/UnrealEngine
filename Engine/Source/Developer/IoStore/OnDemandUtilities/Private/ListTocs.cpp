// Copyright Epic Games, Inc. All Rights Reserved.

#include "Command.h"

#include "Async/Mutex.h"
#include "Async/ParallelFor.h"
#include "Async/UniqueLock.h"
#include "HAL/FileManager.h"
#include "IO/IoContainerHeader.h"
#include "IO/IoStoreOnDemand.h"
#include "Misc/EnumClassFlags.h"
#include "Misc/FileHelper.h"
#include "Misc/Paths.h"
#include "S3/S3Client.h"
#include "Serialization/JsonSerializer.h"
#include "Serialization/JsonWriter.h"
#include "Serialization/MemoryReader.h"

namespace UE::IoStore::Tool
{

using FJsonWriter = TSharedPtr<TJsonWriter<TCHAR, TPrettyJsonPrintPolicy<TCHAR>>>; 
using FJsonWriterFactory = TJsonWriterFactory<TCHAR, TPrettyJsonPrintPolicy<TCHAR>>;

////////////////////////////////////////////////////////////////////////////////

/**
 * Parameters for listing uploaded TOC file(s) from an S3 compatible endpoint.
 *
 * Example usage:
 *
 * 1) Print available TOC's from a local server to standard out.
 * IasTool.exe ListTocs -ServiceUrl="http://10.24.101.92:9000" -Bucket=<bucketname> -BucketPrefix=<some/path/to/data> -AccessKey=<accesskey> -SecretKey=<secretkey>
 *
 * 2) Print available TOC's from AWS S3.
 * IasTool.exe ListTocs -Region="us-east-1" -BucketPath=<mybucket/some/data/path/> -CredentialsFile=<path/to/credentials.txt> -BuildVersion=<version> -Json=<path/to/file.json>
 * 
 * 3) Serialize all TOC's matching a specific build version to JSON:
 * IasTool.exe ListTocs -Region="us-east-1" -BucketPath=<mybucket/some/data/path/> -CredentialsFile=<path/to/credentials.txt> -BuildVersion=<version> -Json=<path/to/file.json>
 *
 * 4) Serialize all chunk object key(s) to JSON.
 * IasTool.exe ListTocs -Region="us-east-1" -BucketPath=<mybucket/some/data/path/> -CredentialsFile=<path/to/credentials.txt> -BuildVersion=<version> -ChunkKeys=<path/to/file.json>
 *
 * 5) Fetch a TOC from a public CDN.
 * IasTool.exe ListTocs -TocUrl=<http://some.public.endpoint.net/path/to/1a32076ca12bfc6feb982ffb064d18f28156606c.iochunktoc>
 *
 * Parameters: -TocEntries, -BlockSizes and -BlockHashes controls what to include when serializing TOC's to JSON.
 *
 * Credentials file example:
 *
 * [default]
 * aws_access_key_id="<key>"
 * aws_secret_access_key="<key>
 * aws_session_token="<token>"
 *
 * Note: All values must be surrounded with "".
 */

////////////////////////////////////////////////////////////////////////////////

enum class EOnDemandTocJsonOptions : uint32
{
	Header			= (1 << 0),
	TocEntries		= (1 << 1),
	BlockSizes		= (1 << 2),
	BlockHashes		= (1 << 3),
	ContainerHeader	= (1 << 4),

	All				= Header | TocEntries | BlockSizes | BlockHashes | ContainerHeader
};
ENUM_CLASS_FLAGS(EOnDemandTocJsonOptions);

EOnDemandTocJsonOptions LexFromContext(const FContext& Context)
{
	if (Context.Get<bool>(TEXT("-AllFilters"), false))
	{
		return EOnDemandTocJsonOptions::All;
	}

	EOnDemandTocJsonOptions JsonOptions = EOnDemandTocJsonOptions::Header;

	if (Context.Get<bool>(TEXT("-TocEntries"), false))
	{
		JsonOptions |= EOnDemandTocJsonOptions::TocEntries;
	}

	if (Context.Get<bool>(TEXT("-BlockSizes"), false))
	{
		JsonOptions |= EOnDemandTocJsonOptions::BlockSizes;
	}

	if (Context.Get<bool>(TEXT("-BlockSizes"), false))
	{
		JsonOptions |= EOnDemandTocJsonOptions::BlockHashes;
	}

	if (Context.Get<bool>(TEXT("-ContainerHeader"), false))
	{
		JsonOptions |= EOnDemandTocJsonOptions::ContainerHeader;
	}

	return JsonOptions;
}

////////////////////////////////////////////////////////////////////////////////

struct FListTocsParams
{
	FString OutFile;
	FString ServiceUrl;
	FString Bucket;
	FString BucketPrefix;
	FString Region; 
	FString AccessKey;
	FString SecretKey;
	FString SessionToken;
	FString CredentialsFile;
	FString CredentialsFileKeyName;
	FString TocUrl;
	FString TocPath;
	FString TocKey;
	FString BuildVersion;
	FString TargetPlatform;
	FString ChunkKeys;
	EOnDemandTocJsonOptions JsonOptions = EOnDemandTocJsonOptions::Header;

	FIoStatus Validate() const;
};

FIoStatus FListTocsParams::Validate() const
{
	if (!TocUrl.IsEmpty())
	{
		return FIoStatus::Ok;
	}
	if (!TocPath.IsEmpty())
	{
		return FIoStatus::Ok;
	}
	if (Bucket.IsEmpty())
	{
		return FIoStatus(EIoErrorCode::InvalidParameter, TEXT("Invalid bucket name"));
	}
	if (!AccessKey.IsEmpty() && SecretKey.IsEmpty())
	{
		return FIoStatus(EIoErrorCode::InvalidParameter, TEXT("Invalid secret key"));
	}
	else if (AccessKey.IsEmpty() && !SecretKey.IsEmpty())
	{
		return FIoStatus(EIoErrorCode::InvalidParameter, TEXT("Invalid access key"));
	}

	if (!CredentialsFile.IsEmpty() && CredentialsFileKeyName.IsEmpty())
	{
		return FIoStatus(EIoErrorCode::InvalidParameter, TEXT("Invalid credential file key name"));
	}

	if (TocUrl.IsEmpty() && ServiceUrl.IsEmpty() && Region.IsEmpty())
	{
		return FIoStatus(EIoErrorCode::InvalidParameter, TEXT("Service URL or AWS region needs to be specified"));
	}

	return FIoStatus::Ok;
}

////////////////////////////////////////////////////////////////////////////////
static FListTocsParams BuildListTocsParams(const FContext& Context)
{
	FListTocsParams Ret;

	if (auto Value = Context.Get<FStringView>(TEXT("-BucketPath")); !Value.IsEmpty())
	{
		int32 Idx = INDEX_NONE;
		if (Value.FindChar(TCHAR('/'), Idx))
		{
			Ret.Bucket = Value.Left(Idx);
			FStringView BucketPrefix = Value.RightChop(Idx + 1);
			if (BucketPrefix.EndsWith(TEXT("/")))
			{
				BucketPrefix.LeftChopInline(1);
			}
			Ret.BucketPrefix = BucketPrefix;
		}
		else
		{
			Ret.Bucket = Value;
		}
	}

	Ret.OutFile					= Context.Get<FStringView>(TEXT("-Json"),					Ret.OutFile);
	Ret.Bucket					= Context.Get<FStringView>(TEXT("-Bucket"),					Ret.Bucket);
	Ret.BucketPrefix			= Context.Get<FStringView>(TEXT("-BucketPrefix"),			Ret.BucketPrefix);
	Ret.ServiceUrl				= Context.Get<FStringView>(TEXT("-ServiceUrl"),				Ret.ServiceUrl);
	Ret.Region					= Context.Get<FStringView>(TEXT("-Region"),					Ret.Region);
	Ret.AccessKey				= Context.Get<FStringView>(TEXT("-AccessKey"),				Ret.AccessKey);
	Ret.SecretKey				= Context.Get<FStringView>(TEXT("-SecretKey"),				Ret.SecretKey);
	Ret.SessionToken			= Context.Get<FStringView>(TEXT("-SessionToken"),			Ret.SessionToken);
	Ret.CredentialsFile			= Context.Get<FStringView>(TEXT("-CredentialsFile"),		Ret.CredentialsFile);
	Ret.CredentialsFileKeyName	= Context.Get<FStringView>(TEXT("-CredentialsFileKeyName"),	Ret.CredentialsFileKeyName);
	Ret.TocUrl					= Context.Get<FStringView>(TEXT("-TocUrl"),					Ret.TocUrl);
	Ret.TocPath					= Context.Get<FStringView>(TEXT("-TocPath"),				Ret.TocPath);
	Ret.TocKey					= Context.Get<FStringView>(TEXT("-TocKey"),					Ret.TocKey);
	Ret.BuildVersion			= Context.Get<FStringView>(TEXT("-BuildVersion"),			Ret.BuildVersion);
	Ret.TargetPlatform			= Context.Get<FStringView>(TEXT("-TargetPlatform"),			Ret.TargetPlatform);
	Ret.ChunkKeys				= Context.Get<FStringView>(TEXT("-ChunkKeys"),				Ret.ChunkKeys);

	Ret.JsonOptions = LexFromContext(Context);

	if (!Ret.TocUrl.IsEmpty())
	{
		FStringView View = Ret.TocUrl;
		if (View.StartsWith(TEXTVIEW("http://")) && View.EndsWith(TEXTVIEW(".iochunktoc")))
		{
			int32 Delim = INDEX_NONE;
			if (View.RightChop(7).FindChar(TEXT('/'), Delim))
			{
				Ret.ServiceUrl = FString(View.Left(7 +  Delim));
				Ret.TocKey = View.RightChop(Ret.ServiceUrl.Len() + 1);
			}
		}
	}

	if (FIoStatus Validation = Ret.Validate(); !Validation.IsOk())
	{
		FString Reason = Validation.ToString();
		Context.Abort(*Reason);
	}

	return Ret;
}

////////////////////////////////////////////////////////////////////////////////
static void GetChunkObjectKeys(const FOnDemandToc& Toc, FString Prefix, TArray<FString>& OutKeys)
{
	if (Prefix.EndsWith(TEXT("/")))
	{
		Prefix = Prefix.Left(Prefix.Len() - 1);
	}

	TStringBuilder<256> Sb;
	for (const FOnDemandTocContainerEntry& Container : Toc.Containers)
	{
		for (const FOnDemandTocEntry& Entry : Container.Entries)
		{
			Sb.Reset();
			const FString Hash = LexToString(Entry.Hash);
			Sb << Prefix << TEXT("/") << Toc.Header.ChunksDirectory << TEXT("/") << Hash.Left(2) << TEXT("/") << Hash << TEXT(".iochunk");
			OutKeys.Add(FString(Sb.ToString()).ToLower());
		}
	}
}


////////////////////////////////////////////////////////////////////////////////
static void ToJson(FJsonWriter JsonWriter, const FOnDemandToc& Toc, EOnDemandTocJsonOptions Options)
{
	JsonWriter->WriteObjectStart(TEXT("Header"));
	{
		JsonWriter->WriteValue(TEXT("Magic"), Toc.Header.Magic);
		JsonWriter->WriteValue(TEXT("Version"), uint64(Toc.Header.Version));
		JsonWriter->WriteValue(TEXT("BlockSize"), uint64(Toc.Header.BlockSize));
		JsonWriter->WriteValue(TEXT("CompressionFormat"), Toc.Header.CompressionFormat);
		JsonWriter->WriteValue(TEXT("ChunksDirectory"), Toc.Header.ChunksDirectory);
		JsonWriter->WriteValue(TEXT("HostGroupName"), Toc.Header.HostGroupName);
	}
	JsonWriter->WriteObjectEnd();
	JsonWriter->WriteObjectStart(TEXT("Meta"));
	{
		const FDateTime Dt = FDateTime::FromUnixTimestamp(Toc.Meta.EpochTimestamp);
		JsonWriter->WriteValue(TEXT("DateTime"), Dt.ToString()), 
		JsonWriter->WriteValue(TEXT("BuildVersion"), Toc.Meta.BuildVersion);
		JsonWriter->WriteValue(TEXT("TargetPlatform"), Toc.Meta.TargetPlatform);
	}
	JsonWriter->WriteObjectEnd();
	JsonWriter->WriteArrayStart(TEXT("Containers"));
	{
		for (const FOnDemandTocContainerEntry& Container : Toc.Containers)
		{
			JsonWriter->WriteObjectStart();
			JsonWriter->WriteValue(TEXT("Name"), Container.ContainerName);
			JsonWriter->WriteValue(TEXT("EncryptionKeyGuid"), Container.EncryptionKeyGuid);
			JsonWriter->WriteValue(TEXT("UTocHash"), LexToString(Container.UTocHash));

			if (EnumHasAnyFlags(Options, EOnDemandTocJsonOptions::ContainerHeader))
			{
				JsonWriter->WriteObjectStart(TEXT("Header"));
				{
					FIoContainerHeader ContainerHeader;
					FMemoryReaderView Ar(Container.Header);
					Ar << ContainerHeader;

					JsonWriter->WriteValue(TEXT("ContainerId"), LexToString(ContainerHeader.ContainerId));
					
					JsonWriter->WriteArrayStart("PackageIds");
					for (const FPackageId& PackageId : ContainerHeader.PackageIds)
					{
						JsonWriter->WriteValue(LexToString(PackageId));
					}
					JsonWriter->WriteArrayEnd();

					JsonWriter->WriteArrayStart("OptionalSegmentPackageIds");
					for (const FPackageId& PackageId : ContainerHeader.OptionalSegmentPackageIds)
					{
						JsonWriter->WriteValue(LexToString(PackageId));
					}
					JsonWriter->WriteArrayEnd();
				}
				JsonWriter->WriteObjectEnd();
			}

			if (EnumHasAnyFlags(Options, EOnDemandTocJsonOptions::TocEntries))
			{
				JsonWriter->WriteArrayStart("Entries");
				for (const FOnDemandTocEntry& Entry : Container.Entries)
				{
					JsonWriter->WriteObjectStart();
					JsonWriter->WriteValue(TEXT("Hash"), LexToString(Entry.Hash));
					JsonWriter->WriteValue(TEXT("ChunkId"), LexToString(Entry.ChunkId));
					JsonWriter->WriteValue(TEXT("RawSize"), Entry.RawSize);
					JsonWriter->WriteValue(TEXT("EncodedSize"), Entry.EncodedSize);
					JsonWriter->WriteValue(TEXT("BlockOffset"), int32(Entry.BlockOffset));
					JsonWriter->WriteValue(TEXT("BlockCount"), int32(Entry.BlockCount));
					JsonWriter->WriteObjectEnd();
				}
				JsonWriter->WriteArrayEnd();
			}

			if (EnumHasAnyFlags(Options, EOnDemandTocJsonOptions::BlockSizes))
			{
				JsonWriter->WriteArrayStart("Blocks");
				{
					for (uint32 BlockSize : Container.BlockSizes)
					{
						JsonWriter->WriteValue(int32(BlockSize));
					}
				}
				JsonWriter->WriteArrayEnd();
			}

			if (EnumHasAnyFlags(Options, EOnDemandTocJsonOptions::BlockHashes))
			{
				JsonWriter->WriteArrayStart("BlockHashes");
				{
					for (const FIoBlockHash& BlockHash : Container.BlockHashes)
					{
						JsonWriter->WriteValue(int32(BlockHash));
					}
				}
				JsonWriter->WriteArrayEnd();
			}
			JsonWriter->WriteObjectEnd();
		}
	}
	JsonWriter->WriteArrayEnd();
}

////////////////////////////////////////////////////////////////////////////////
FIoStatus ListTocs(const FListTocsParams& Params)
{
	struct FChunkStats
	{
		TSet<FIoHash> Chunks;
		uint64 TotalChunkSize = 0;

		void Add(const FIoHash& Hash, uint64 ChunkSize)
		{
			bool bExist = false;
			Chunks.Add(Hash, &bExist);
			if (!bExist)
			{
				TotalChunkSize += ChunkSize;
			}
		}
	};
	TMap<FString, FChunkStats> Stats;

	struct FTocDescription
	{
		FOnDemandToc Toc;
		FDateTime DateTime;
		FString Key;
		uint64 Size = 0;
		uint64 ChunkCount = 0;
		uint64 TotalChunkSize = 0;
	};

	const bool bFilteredQuery = !Params.TocKey.IsEmpty() || !Params.BuildVersion.IsEmpty() || !Params.TargetPlatform.IsEmpty();

	FS3ClientConfig Config;
	Config.ServiceUrl = Params.ServiceUrl;
	Config.Region = Params.Region;
	
	FS3ClientCredentials Credentials;
	if (Params.CredentialsFile.IsEmpty() == false)
	{
		UE_LOGF(LogIas, Display, "Loading credentials file '%ls'", *Params.CredentialsFile);
		FS3CredentialsProfileStore CredentialsStore = FS3CredentialsProfileStore::FromFile(Params.CredentialsFile);
		if (CredentialsStore.TryGetCredentials(Params.CredentialsFileKeyName, Credentials) == false)
		{
			return FIoStatus(EIoErrorCode::InvalidParameter, TEXT("Failed to find valid credentials in credentials file"));
		}
		else
		{
			UE_LOGF(LogIas, Display, "Found credentials for '%ls'", *Params.CredentialsFileKeyName);
		}
	}
	else
	{
		Credentials = FS3ClientCredentials(Params.AccessKey, Params.SecretKey, Params.SessionToken);
	}

	FS3Client Client(Config, Credentials);
	FMutex TocMutex;
	TArray<FTocDescription> Tocs;

	if (!Params.TocPath.IsEmpty())
	{
		UE_LOGF(LogIas, Display, "Loading TOC from file '%ls'", *Params.TocPath);

		const int64 FileSize = IFileManager::Get().FileSize(*Params.TocPath);
		if (FileSize <= 0)
		{
			return FIoStatus(EIoErrorCode::NotFound);
		}

		TIoStatusOr<FOnDemandToc> TocStatus = FOnDemandToc::LoadFromFile(Params.TocPath, false);
		if (!TocStatus.IsOk())
		{
			return TocStatus.Status();
		}

		FOnDemandToc Toc = TocStatus.ConsumeValueOrDie();
		if (Toc.Header.Magic == FOnDemandTocHeader::ExpectedMagic)
		{
			FDateTime DateTime = FDateTime::FromUnixTimestamp(Toc.Meta.EpochTimestamp);
			Tocs.Add(FTocDescription 
			{
				.Toc = MoveTemp(Toc),
				.DateTime = DateTime,
				.Size = static_cast<uint64>(FileSize)
			});
		}
		else
		{
			return FIoStatus(EIoErrorCode::FileOpenFailed, TEXT("Header mismatch"));
		}
	}
	else if (Params.TocUrl.IsEmpty())
	{
		FStringView PrefixView = Params.BucketPrefix;
		if (PrefixView.StartsWith(TEXT("/")))
		{
			PrefixView.RightChopInline(1);
		}

		TStringBuilder<256> Path;
		if (!PrefixView.IsEmpty())
		{
			Path << PrefixView << TEXT("/");
		}

		UE_LOGF(LogIas, Display, "Fetching TOC's from '%ls/%ls/%ls'", *Client.GetConfig().ServiceUrl, *Params.Bucket, Path.ToString());
		FS3ListObjectResponse Response = Client.ListObjects(FS3ListObjectsRequest
		{
			Params.Bucket,
			Path.ToString(),
			TEXT('/')
		});

		if (Response.Objects.IsEmpty())
		{
			UE_LOGF(LogIas, Display, "Not TOC's found at '%ls/%ls/%ls' (%ls)", *Client.GetConfig().ServiceUrl, *Params.Bucket, Path.ToString(), *Response.GetErrorStatus());
			return FIoStatus(EIoErrorCode::NotFound);
		}

		ParallelFor(Response.Objects.Num(), [&Params, &Client, &Response, &Tocs, &TocMutex](int32 Index)
		{
			const FS3Object& Obj = Response.Objects[Index];
			if (Obj.Key.EndsWith(TEXT("iochunktoc")) == false)
			{
				return;
			}

			if (!Params.TocKey.IsEmpty() && !Params.TocKey.Equals(FPaths::GetBaseFilename(Obj.Key), ESearchCase::IgnoreCase))
			{
				return;
			}

			UE_LOGF(LogIas, Display, "Fetching TOC '%ls/%ls/%ls'", *Client.GetConfig().ServiceUrl, *Params.Bucket, *Obj.Key);
			FS3GetObjectResponse TocResponse = Client.GetObject(FS3GetObjectRequest
			{
				Params.Bucket,
				Obj.Key
			});

			if (TocResponse.IsOk() == false)
			{
				UE_LOGF(LogIas, Warning, "Failed to fetch TOC '%ls/%ls/%ls'", *Client.GetConfig().ServiceUrl, *Params.Bucket, *Obj.Key);
				return;
			}

			FOnDemandToc Toc;
			FMemoryReaderView Ar(TocResponse.GetBody().GetView());
			Ar << Toc;

			if (Ar.IsError() || Toc.Header.Magic != FOnDemandTocHeader::ExpectedMagic)
			{
				UE_LOGF(LogIas, Warning, "Failed to serialize TOC '%ls/%ls/%ls'. Header version/magic mimsatch", *Client.GetConfig().ServiceUrl, *Params.Bucket, *Obj.Key);
				return;
			}

			if (!Params.BuildVersion.IsEmpty() && !Params.BuildVersion.Equals(Toc.Meta.BuildVersion, ESearchCase::IgnoreCase))
			{
				return;
			}

			if (!Params.TargetPlatform.IsEmpty() && !Params.TargetPlatform.Equals(Toc.Meta.TargetPlatform, ESearchCase::IgnoreCase))
			{
				return;
			}

			FDateTime DateTime = FDateTime::FromUnixTimestamp(Toc.Meta.EpochTimestamp);
			FTocDescription Desc = FTocDescription 
			{
				.Toc = MoveTemp(Toc),
				.DateTime = DateTime,
				.Key = Obj.Key,
				.Size = Obj.Size
			};

			{
				TUniqueLock Lock(TocMutex);
				Tocs.Add(MoveTemp(Desc));
			}
		});
	}
	else
	{
		UE_LOGF(LogIas, Display, "Fetching TOC '%ls/%ls'", *Client.GetConfig().ServiceUrl, *Params.TocKey);
		FS3GetObjectResponse TocResponse = Client.GetObject(FS3GetObjectRequest
		{
			.Key = Params.TocKey
		});

		if (TocResponse.IsOk())
		{
			FOnDemandToc Toc;
			FMemoryReaderView Ar(TocResponse.GetBody().GetView());
			Ar << Toc;

			if (Toc.Header.Magic == FOnDemandTocHeader::ExpectedMagic)
			{
				FDateTime DateTime = FDateTime::FromUnixTimestamp(Toc.Meta.EpochTimestamp);
				Tocs.Add(FTocDescription 
				{
					.Toc = MoveTemp(Toc),
					.DateTime = DateTime,
					.Key = Params.TocKey,
					.Size = TocResponse.GetBody().GetSize()
				});
			}
			else
			{
				UE_LOGF(LogIas, Warning, "Failed to serialize TOC '%ls/%ls'. Header magic mimsatch", *Client.GetConfig().ServiceUrl, *Params.TocKey);
			}
		}
	}

	if (Tocs.IsEmpty())
	{
		return FIoStatus(EIoErrorCode::NotFound);
	}

	for (FTocDescription& Desc : Tocs)
	{
		FChunkStats& BuildStats = Stats.FindOrAdd(Desc.Toc.Meta.BuildVersion);
		FChunkStats& BucketStats = Stats.FindOrAdd(Params.Bucket);
		for (const FOnDemandTocContainerEntry& Container : Desc.Toc.Containers)
		{
			for (const FOnDemandTocEntry& Entry : Container.Entries)
			{
				Desc.ChunkCount++;
				Desc.TotalChunkSize += Entry.EncodedSize;
				BucketStats.Add(Entry.Hash, Entry.EncodedSize);
				BuildStats.Add(Entry.Hash, Entry.EncodedSize);
			}
		}
	}

	Tocs.Sort([](const auto& LHS, const auto& RHS) { return LHS.DateTime > RHS.DateTime; });

	if (Params.OutFile.IsEmpty())
	{
		int32 Counter = 1;
		TStringBuilder<256> Url;

		for (const FTocDescription& Desc : Tocs)
		{
			Url.Reset();
			Url << Client.GetConfig().ServiceUrl << TEXT("/") << Params.Bucket << TEXT("/") << Desc.Key;

			UE_LOGF(LogIas, Display, "");
			UE_LOGF(LogIas, Display, "%d) %ls", Counter++, *Desc.Key);
			UE_LOGF(LogIas, Display, "%-20ls: %ls",TEXT("Date"), *Desc.DateTime.ToString());
			UE_LOGF(LogIas, Display, "%-20ls: %ls", TEXT("BuildVersion"), *Desc.Toc.Meta.BuildVersion);
			UE_LOGF(LogIas, Display, "%-20ls: %ls", TEXT("TargetPlatform"), *Desc.Toc.Meta.TargetPlatform);
			UE_LOGF(LogIas, Display, "%-20ls: %.2lf KiB", TEXT("TocSize"), double(Desc.Size) / 1024.0);
			UE_LOGF(LogIas, Display, "%-20ls: %llu", TEXT("ChunkCount"), Desc.ChunkCount);
			UE_LOGF(LogIas, Display, "%-20ls: %.2lf MiB", TEXT("TotalChunkSize"), double(Desc.TotalChunkSize) / 1024.0 / 1024.0);
			if (Params.TocPath.IsEmpty())
			{
				UE_LOGF(LogIas, Display, "%-20ls: %ls", TEXT("Url"), Url.ToString());
			}
			else
			{
				UE_LOGF(LogIas, Display, "%-20ls: %ls", TEXT("FilePath"), *Params.TocPath);
			}
		}
		UE_LOGF(LogIas, Display, "");
		
		if (bFilteredQuery == false)
		{
			TArray<FString> Keys;
			Stats.GetKeys(Keys);
			Keys.Sort();

			UE_LOGF(LogIas, Display, "%-80ls %15ls %15ls", TEXT("BuildVersion"), TEXT("Chunk(s)"), TEXT("MiB"));
			UE_LOGF(LogIas, Display, "-------------------------------------------------------------------------------------------------------------------");
			for (const FString& Key : Keys)
			{
				if (Key != Params.Bucket)
				{
					const FChunkStats& ChunkStats = Stats.FindChecked(Key);
					UE_LOGF(LogIas, Display, "%-80ls %15d %15.2f", *Key, ChunkStats.Chunks.Num(), double(ChunkStats.TotalChunkSize) / 1024.0 / 1024.0);
				}
			}
			UE_LOGF(LogIas, Display, "-------------------------------------------------------------------------------------------------------------------");
			FChunkStats& BucketStats = Stats.FindOrAdd(Params.Bucket);
			UE_LOGF(LogIas, Display, "%-80ls %15d %15.2f", *Params.Bucket, BucketStats.Chunks.Num(), double(BucketStats.TotalChunkSize) / 1024.0 / 1024.0);
			UE_LOGF(LogIas, Display, "");
		}
	}
	else
	{
		FString Json;
		FJsonWriter JsonWriter = FJsonWriterFactory::Create(&Json);
		JsonWriter->WriteObjectStart();
		JsonWriter->WriteValue(TEXT("ServiceUrl"), Client.GetConfig().ServiceUrl);
		JsonWriter->WriteValue(TEXT("Bucket"), Params.Bucket);
		JsonWriter->WriteValue(TEXT("BucketPrefix"), Params.BucketPrefix);

		JsonWriter->WriteArrayStart(TEXT("Tocs"));
		for (const FTocDescription& Desc : Tocs)
		{
			JsonWriter->WriteObjectStart();
			JsonWriter->WriteValue(TEXT("Key"), Desc.Key);
			JsonWriter->WriteValue(TEXT("ChunkCount"), Desc.ChunkCount);
			JsonWriter->WriteValue(TEXT("TotalChunkSize"), Desc.TotalChunkSize);

			ToJson(JsonWriter, Desc.Toc, Params.JsonOptions);
			JsonWriter->WriteObjectEnd();
		}
		JsonWriter->WriteArrayEnd();

		JsonWriter->WriteObjectEnd();
		JsonWriter->Close();

		UE_LOGF(LogIas, Display, "Saving file '%ls'", *Params.OutFile);
		if (!FFileHelper::SaveStringToFile(Json, *Params.OutFile))
		{
			return FIoStatus(EIoErrorCode::WriteError, TEXTVIEW("Failed writing JSON file")); 
		}
	}

	if (Params.ChunkKeys.IsEmpty() == false)
	{
		FString Json;
		FJsonWriter JsonWriter = FJsonWriterFactory::Create(&Json);
		
		JsonWriter->WriteObjectStart();
		JsonWriter->WriteValue(TEXT("ServiceUrl"), Client.GetConfig().ServiceUrl);
		JsonWriter->WriteValue(TEXT("Bucket"), Params.Bucket);
		JsonWriter->WriteValue(TEXT("BucketPrefix"), Params.BucketPrefix);

		JsonWriter->WriteArrayStart(TEXT("Tocs"));
		for (const FTocDescription& Desc : Tocs)
		{
			TArray<FString> ObjKeys;
			GetChunkObjectKeys(Desc.Toc, Params.BucketPrefix, ObjKeys);

			JsonWriter->WriteObjectStart();
			JsonWriter->WriteValue(TEXT("Key"), Desc.Key);
			JsonWriter->WriteValue(TEXT("BuildVersion"), Desc.Toc.Meta.BuildVersion);
			JsonWriter->WriteValue(TEXT("TargetPlatform"), Desc.Toc.Meta.TargetPlatform);
			JsonWriter->WriteArrayStart(TEXT("ChunkKeys"));
			for (const FString& Key : ObjKeys)
			{
				JsonWriter->WriteValue(Key);
			}
			JsonWriter->WriteArrayEnd();
			JsonWriter->WriteObjectEnd();
		}
		JsonWriter->WriteArrayEnd();

		JsonWriter->WriteObjectEnd();
		JsonWriter->Close();

		UE_LOGF(LogIas, Display, "Saving chunk key(s) '%ls'", *Params.ChunkKeys);
		if (!FFileHelper::SaveStringToFile(Json, *Params.ChunkKeys))
		{
			return FIoStatus(EIoErrorCode::WriteError, TEXTVIEW("Failed writing JSON file")); 
		}
	}

	return FIoStatus::Ok;
}

////////////////////////////////////////////////////////////////////////////////
static int32 ListTocsCommandEntry(const Tool::FContext& Context)
{
	FListTocsParams Params = BuildListTocsParams(Context);

	FIoStatus Status = ListTocs(Params);
	if (!Status.IsOk())
	{
		FString Reason = Status.ToString();
		Context.Abort(*Reason);
	}

	return 0;
}

////////////////////////////////////////////////////////////////////////////////
extern FArgumentSet S3Arguments;

static FCommand ListTocsCommand(
	ListTocsCommandEntry,
	TEXT("ListTocs"),
	TEXT("Enumerates and output information for available TOCs"),
	{
		TArgument<FStringView>(TEXT("-Json"),			TEXT("Optionally write output to the given JSON file")),
		TArgument<FStringView>(TEXT("-TocUrl"),			TEXT("URL of a TOC file to fetch info of (overrides Bucket/ServiceUrl)")),
		TArgument<FStringView>(TEXT("-TocPath"),		TEXT("Path to a .uondemandtoc file on disk")),
		TArgument<FStringView>(TEXT("-TocKey"),			TEXT("Filter output by TOC key")),
		TArgument<FStringView>(TEXT("-BuildVersion"),	TEXT("Filter results a particular build version")),
		TArgument<FStringView>(TEXT("-TargetPlatform"),	TEXT("Only show results for a given platform")),
		TArgument<FStringView>(TEXT("-ChunkKeys"),		TEXT("Path to write JSON file of chunk keys to")),
		TArgument<bool>(TEXT("-AllFilters"),			TEXT("Enabled all filters in JSON output")),
		TArgument<bool>(TEXT("-ContainerHeader"),		TEXT("Add the container header info in JSON")),
		TArgument<bool>(TEXT("-TocEntries"),			TEXT("Include TOC entries in JSON output")),
		TArgument<bool>(TEXT("-BlockSizes"),			TEXT("Write block sizes in JSON output")),
		TArgument<bool>(TEXT("-BlockHashes"),			TEXT("Add block hash values in JSON output")),

		TArgument<FStringView>(TEXT("-BucketPrefix"),	TEXT("Bucket path to enumerate")),
		TArgument<FStringView>(TEXT("-BucketPath"),		TEXT("Alternative way to set Bucket/BucketPrefix")),
		S3Arguments,
	}
);

} // namespace UE::IoStore::Tool
