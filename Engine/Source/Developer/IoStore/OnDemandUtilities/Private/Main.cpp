// Copyright Epic Games, Inc. All Rights Reserved.

#include "Command.h"

namespace UE::IoStore::Tool {

////////////////////////////////////////////////////////////////////////////////
FArgumentSet S3Arguments = {
	TArgument<FStringView>(TEXT("-ServiceUrl"),				TEXT("S3 service URL")),
	TArgument<FStringView>(TEXT("-Bucket"),					TEXT("S3 bucket name")),
	TArgument<FStringView>(TEXT("-Region"),					TEXT("S3 region code")),
	TArgument<FStringView>(TEXT("-AccessKey"),				TEXT("S3 access key")),
	TArgument<FStringView>(TEXT("-SecretKey"),				TEXT("S3 secret key")),
	TArgument<FStringView>(TEXT("-SessionToken"),			TEXT("S3 session token")),
	TArgument<FStringView>(TEXT("-CredentialsFile"),		TEXT("S3 credentials file")),
	TArgument<FStringView>(TEXT("-CredentialsFileKeyName"),	TEXT("S3 credentials to use from file")),
};

////////////////////////////////////////////////////////////////////////////////
int32 Main(int32 ArgC, TCHAR* ArgV[])
{
	return FCommand::Main(ArgC, ArgV);
}

} // namespace UE::IoStore::Tool
