// Copyright Epic Games, Inc. All Rights Reserved.

#include "Command.h"

#include <Containers/UnrealString.h>
#include <HAL/FileManager.h>
#include <HAL/PlatformProcess.h>
#include <IO/IoBuffer.h>
#include <IO/Http/Client.h>
#include <Misc/Paths.h>
#include <Misc/ScopeExit.h>

#if PLATFORM_WINDOWS
#	include <Windows/AllowWindowsPlatformTypes.h>
#		include <winsock2.h>
#		include <ws2tcpip.h>
#	include <Windows/HideWindowsPlatformTypes.h>
#endif

#include <cstdio>

namespace UE::IoStore::Tool
{

////////////////////////////////////////////////////////////////////////////////
static void LoadCaCerts(bool bSystemStore)
{
	using namespace UE::IoStore::HTTP;

	if (bSystemStore)
	{
		FMemoryView NoData;
		FCertRoots CaRoots(NoData);
		if (CaRoots.IsValid())
		{
			FCertRoots::SetDefault(MoveTemp(CaRoots));
			return;
		}
	}

	IFileManager& Ifm = IFileManager::Get();
	FString PemPath = FPaths::EngineContentDir() / TEXT("Certificates/ThirdParty/cacert.pem");
	FArchive* Reader = Ifm.CreateFileReader(*PemPath);
	check(Reader != nullptr)

	uint32 Size = uint32(Reader->TotalSize());
	FIoBuffer PemData(Size);
	FMutableMemoryView PemView = PemData.GetMutableView();
	Reader->Serialize(PemView.GetData(), Size);

	FCertRoots CaRoots(PemData.GetView());
	check(CaRoots.IsValid());

	FCertRoots::SetDefault(MoveTemp(CaRoots));
	delete Reader;
}

////////////////////////////////////////////////////////////////////////////////
static int32 PurlCommandEntry(const FContext& Context)
{
#if PLATFORM_WINDOWS
	WSADATA WsaData;
	if (WSAStartup(MAKEWORD(2, 2), &WsaData) == 0x0a9e0493)
		return 1;
	ON_SCOPE_EXIT { WSACleanup(); };
#endif

	using namespace UE::IoStore::HTTP;

	{
		bool bSystemStore = Context.Get<bool>(TEXT("-SystemCA"));
		LoadCaCerts(bSystemStore);
	}

	FStringView Url = Context.Get<FStringView>(TEXT("Url"));
	auto AnsiUrl = StringCast<ANSICHAR>(Url.GetData(), Url.Len());

	FString Method(Context.Get<FStringView>(TEXT("-Method"), TEXT("GET")));
	Method = Method.ToUpper();
	auto AnsiMethod = StringCast<ANSICHAR>(*Method);

	bool bChunked = false;
	uint32 ContentSize = 0;
	auto Sink = [Dest=FIoBuffer(), &ContentSize, &bChunked] (const FTicketStatus& Status) mutable
	{
		if (Status.GetId() == FTicketStatus::EId::Response)
		{
			FResponse& Response = Status.GetResponse();
			FAnsiStringView Message = Response.GetStatusMessage();
			std::printf("%u %.*s\n",
				Response.GetStatusCode(),
				Message.Len(),
				Message.GetData()
			);
			Response.ReadHeaders([] (FAnsiStringView Name, FAnsiStringView Value)
			{
				std::printf("%.*s: %.*s\n",
					Name.Len(),
					Name.GetData(),
					Value.Len(),
					Value.GetData()
				);
				return true;
			});

			bChunked = (Response.GetContentLength() == -1);
			Response.SetDestination(&Dest);
			return;
		}

		if (Status.GetId() == FTicketStatus::EId::Content)
		{
			ContentSize += uint32(Dest.GetSize());
			return;
		}

		if (Status.GetId() == FTicketStatus::EId::Error)
		{
			const char* Reason = Status.GetError().Reason;
			std::printf("ERROR: %s\n", Reason);
			return;
		}
	};

	bool bRedirect = Context.Get<bool>(TEXT("-Redirect"));
	bool bUseHttp2 = Context.Get<bool>(TEXT("-HttpTwo"));
	int32 BufSizeKiB = Context.Get<int32>(TEXT("-BufSize"), -1);
	uint32 ThrottleKiBps = Context.Get<uint32>(TEXT("-Throttle"), 0);

	FConnectionPool::FParams PoolParams = {
		.RecvBufSize = BufSizeKiB << 10,
		.ConnectionCount = 1,
		.HttpVersion = (bUseHttp2 ? EHttpVersion::Two : EHttpVersion::One),
	};
	if (!PoolParams.SetHostFromUrl(AnsiUrl))
	{
		std::printf("ERROR: invalid url\n");
		return 1;
	}
	FConnectionPool Pool(PoolParams);

	FEventLoop Loop;

	FEventLoop::FRequestParams RequestParams = {
		.bAutoRedirect = bRedirect,
	};

	FAnsiStringView AnsiPath = AnsiUrl;
	for (int32 i = 2; i >= 0; i -= 2)
	{
		int32 Index;
		AnsiPath.FindChar('/', Index);
		AnsiPath = AnsiPath.Mid(Index + i);
	}

	FRequest Request = Loop.Request(AnsiMethod, AnsiPath, Pool, &RequestParams);
	Request.Header("user-agent", "purl");
	Loop.Send(MoveTemp(Request), Sink);

	Loop.SetFailTimeout(4000);
	Loop.Throttle(ThrottleKiBps);
	while (Loop.Tick(-1))
	{
	}

	std::printf("Data: %u bytes\n", ContentSize);

	return 0;
}

////////////////////////////////////////////////////////////////////////////////
static FCommand PurlCommand(
	PurlCommandEntry,
	TEXT("Purl"),
	TEXT("Uses IoStore's HTTP client to download a URL"),
	{
		TArgument<FStringView>(TEXT("Url"), TEXT("Url to download")),
		TArgument<FStringView>(TEXT("-Method"), TEXT("Request method")),
		TArgument<bool>(TEXT("-Redirect"), TEXT("Follow 30x redirects")),
		TArgument<bool>(TEXT("-SystemCA"), TEXT("Load certificates from system CA store")),
		TArgument<bool>(TEXT("-HttpTwo"), TEXT("Use HTTP/2")),
		TArgument<int32>(TEXT("-BufSize"), TEXT("Recv buffer size in KiB")),
		TArgument<uint32>(TEXT("-Throttle"), TEXT("Bandwidth throttle in KiB/s")),
	}
);

} // namespace UE::IoStore::Tool
