// Copyright Epic Games, Inc. All Rights Reserved.

#include "HAL/Platform.h"

#if PLATFORM_WINDOWS

#include "CoreGlobals.h"
#include "Command.h"

#include <Misc/ScopeExit.h>

#if PLATFORM_WINDOWS
#	include <Windows/AllowWindowsPlatformTypes.h>
#		include <winsock2.h>
#		include <ws2tcpip.h>
#	include <Windows/HideWindowsPlatformTypes.h>
#endif

#include <atomic>
#include <cstdio>
#include <thread>

namespace UE::IoStore::Tool
{

////////////////////////////////////////////////////////////////////////////////
struct FSocks5Stats
{
	uint32				Source = 0;
	std::atomic<uint32>	Dest = 0;
	std::atomic<int32>	Id = 0;
	std::atomic<uint64>	Counts[2]; // in KiB
};

////////////////////////////////////////////////////////////////////////////////
static void Socks5Peer(SOCKET Peer, FSocks5Stats* Stats)
{
#ifdef _MSC_VER
	// MSVC's static analysis doesn't see that 'Result' from recv() is checked
	// to be the exact size of the destination buffer.
	#pragma warning(push)
	#pragma warning(disable : 6385)
#endif // _MSC_VER

	ON_SCOPE_EXIT
	{
		closesocket(Peer);
		Stats->Id.store(-1, std::memory_order_release);
	};

	// greeting
	char Buf[32];
	for (int32 i = 0; i < 2; ++i)
	{
		if (recv(Peer, Buf + i, 1, 0) != 1)
		{
			return;
		}
	}
	check(Buf[0] == 5);

	bool bNoAuth = false;
	for (int32 i = 0, n = Buf[1]; i < n; ++i)
	{
		if (recv(Peer, Buf + i, 1, 0) != 1)
		{
			return;
		}
	}
	bNoAuth |= (Buf[0] == 0);
	check(bNoAuth);

	// ack
	Buf[0] = 5; Buf[1] = 0;
	check(send(Peer, Buf, 2, 0) == 2);

	// connect req.
	for (int32 i = 0; i < 10; ++i)
	{
		if (recv(Peer, Buf + i, 1, 0) != 1)
		{
			return;
		}
	}
	check(Buf[0] == 5);
	check(Buf[1] == 1);
	check(Buf[3] == 1);

	uint32 Ip;
	uint16 Port;
	std::memcpy(&Ip, Buf + 4, sizeof(Ip));
	std::memcpy(&Port, Buf + 8, sizeof(Port));
	Stats->Dest.store(Ip);

	SOCKET Remote = socket(AF_INET, SOCK_STREAM, IPPROTO_TCP);
	check(Remote != INVALID_SOCKET);
	ON_SCOPE_EXIT { closesocket(Remote); };

	sockaddr_in Addr = { sizeof(sockaddr_in) };
	Addr.sin_family = AF_INET;
	Addr.sin_addr.s_addr = Ip;
	Addr.sin_port = Port;
	if (connect(Remote, &(sockaddr&)Addr, sizeof(Addr)) != 0)
	{
		return;
	}

	std::memset(Buf, 0, sizeof(Buf));
	Buf[0] = 5;
	Buf[3] = 1;
	check(send(Peer, Buf, 10, 0) == 10);

	static const uint32 BufferSize = 1 << 20;
	char* Buffer = (char*)malloc(BufferSize);
	ON_SCOPE_EXIT { free(Buffer); };

	while (true)
	{
		pollfd Polls[2] = { { Peer, POLLIN }, { Remote, POLLIN } };
		int32 Result = WSAPoll(Polls, 2, 100);
		check(Result >= 0);
		if (Result == 0)
		{
			continue;
		}

		auto Xfer = [&] (SOCKET In, SOCKET Out)
		{
			int32 Size = recv(In, Buffer, BufferSize, 0);
			for (int32 Remaining = Size; Remaining > 0;)
			{
				Result = send(Out, Buffer, Size, 0);
				if (Result <= 0)
				{
					return;
				}
				Remaining -= Size;
			}

			int32 Index = (Out == Remote);
			Stats->Counts[Index].fetch_add(Size >> 10, std::memory_order_relaxed);
		};

		if ((Polls[0].revents & ~POLLIN) || (Polls[1].revents & ~POLLIN))
		{
			break;
		}

		if (Polls[0].revents & POLLIN)
		{
			Xfer(Peer, Remote);
		}

		if (Polls[1].revents & POLLIN)
		{
			Xfer(Remote, Peer);
		}
	}

#ifdef _MSC_VER
	#pragma warning(pop)
#endif // _MSC_VER
}

////////////////////////////////////////////////////////////////////////////////
static void Socks5Server(uint32 Port)
{
	auto PrintIp = [] (uint32 Ip4)
	{
		std::printf(
			"%3u.%3u.%3u.%3u",
			(Ip4 & 0x00'00'00'ff) >>  0,
			(Ip4 & 0x00'00'ff'00) >>  8,
			(Ip4 & 0x00'ff'00'00) >> 16,
			(Ip4 & 0xff'00'00'00) >> 24
		);
	};

	TArray<FSocks5Stats*> AllStats;
	int32 Counter = 0;
	uint64 SoFar[2] = {};

	std::printf("Listening on 0.0.0.0:%u\n\n", Port);

	SOCKET Sock = socket(AF_INET, SOCK_STREAM, IPPROTO_TCP);
	check(Sock != INVALID_SOCKET);

	sockaddr_in Addr = { sizeof(sockaddr_in) };
	Addr.sin_family = AF_INET;
	Addr.sin_addr.s_addr = 0;
	Addr.sin_port = htons(uint16(Port));
	check(bind(Sock, &(sockaddr&)Addr, sizeof(Addr)) == 0);
	check(listen(Sock, 32) == 0);
	for (uint32 Tick = 0; !IsEngineExitRequested();)
	{
		pollfd Poll = { Sock, POLLIN };
		int32 Result = WSAPoll(&Poll, 1, 654);
		if (Result == -1)
		{
			break;
		}

		if (Result == 1)
		{
			int32 AddrSize = sizeof(Addr);
			SOCKET Client = accept(Sock, &(sockaddr&)Addr, &AddrSize);

			FSocks5Stats* PeerStats = new FSocks5Stats();
			PeerStats->Source = Addr.sin_addr.s_addr;
			PeerStats->Id = ++Counter;
			AllStats.Add(PeerStats);
			std::thread(Socks5Peer, Client, PeerStats).detach();
			continue;
		}

		uint64 UpDown[2] = {};
		uint32 LineClears = 0;
		for (int32 i = 0, n = AllStats.Num(); i < n; ++i)
		{
			FSocks5Stats* Stats = AllStats[i];

			uint64 DownKiB = Stats->Counts[0].load(std::memory_order_relaxed);
			uint64 UpKiB = Stats->Counts[1].load(std::memory_order_relaxed);

			if (Stats->Id.load(std::memory_order_acquire) == -1)
			{
				SoFar[0] += UpKiB;
				SoFar[1] += DownKiB;
				delete Stats;
				AllStats[i] = AllStats.Last();
				AllStats.Pop();
				--i, --n, ++LineClears;
				continue;
			}

			std::printf("%04d: ", Stats->Id.load(std::memory_order_relaxed));
			PrintIp(Stats->Source);
			std::printf("  ->  ");
			PrintIp(Stats->Dest.load(std::memory_order_relaxed));
			std::printf(
				" : d:%9llu u:%9llu KiB\n",
				DownKiB,
				UpKiB
			);

			UpDown[0] += DownKiB;
			UpDown[1] += UpKiB;
		}

		for (uint32 i = 0; i < LineClears; ++i)
		{
			std::printf("\x1b[2K\n");
		}
		std::printf("\x1b[%uF", uint32(AllStats.Num()) + LineClears + 1);

		const char* SignOfLife = ".oOo";
		std::printf(
			"[%c] n:%d d:%llu u:%llu\n",
			SignOfLife[Tick++ & 0x3],
			Counter,
			SoFar[0] + UpDown[0],
			SoFar[1] + UpDown[1]
		);
	}

	closesocket(Sock);
}

////////////////////////////////////////////////////////////////////////////////
static int32 SocksCommandEntry(const FContext& Context)
{
	WSADATA WsaData;
	if (WSAStartup(MAKEWORD(2, 2), &WsaData) == 0x0a9e0493)
	{
		return 1;
	}
	ON_SCOPE_EXIT { WSACleanup(); };

	uint32 Port = Context.Get<uint32>(TEXT("-Port"), 24930u);
	std::thread(Socks5Server, Port).join();

	return 0;
}

////////////////////////////////////////////////////////////////////////////////
static FCommand SocksCommand(
	SocksCommandEntry,
	TEXT("Socks"),
	TEXT("Rudimentary SOCKS5 proxy to aid in testing IAS traffic"),
	{
		TArgument<uint32>(TEXT("-Port"), TEXT("Port to listen on (default=24930)")),
	}
);

} // namespace UE::IoStore::Tool

#endif // PLATFORM_WINDOWS
