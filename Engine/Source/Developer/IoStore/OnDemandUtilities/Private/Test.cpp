// Copyright Epic Games, Inc. All Rights Reserved.

#include "Command.h"

#include "Async/TaskGraphInterfaces.h"
#include "Containers/AnsiString.h"
#include "Containers/StringConv.h"
#include "Containers/UnrealString.h"
#include "Misc/CommandLine.h"

namespace UE::IoStore {

////////////////////////////////////////////////////////////////////////////////
namespace IasJournaledFileCacheTest { void Tests(const TCHAR*);					}
namespace HTTP						{ void IasHttpTest(const ANSICHAR*, uint32);}

namespace Tool {

////////////////////////////////////////////////////////////////////////////////
void CommandTest();

////////////////////////////////////////////////////////////////////////////////
static void HttpTests(const FContext& Context)
{
	FAnsiString TestHost = FAnsiString(Context.Get<FStringView>(TEXT("-Host"), TEXT("localhost")));

	uint32 Seed = Context.Get<uint32>(TEXT("-HttpSeed"), 493);
	HTTP::IasHttpTest(*TestHost, Seed);
}

////////////////////////////////////////////////////////////////////////////////
static void CacheTests(const FContext& Context)
{
	FString CacheDirStr = FString(Context.Get<FStringView>(TEXT("-Dir")));

	const TCHAR* CacheDir = CacheDirStr.IsEmpty() ? nullptr : *CacheDirStr;
	IasJournaledFileCacheTest::Tests(CacheDir);
}

////////////////////////////////////////////////////////////////////////////////
static int32 TestCommandEntry(const FContext& Context)
{

	FStringView Only = Context.Get<FStringView>(TEXT("-Only"));
#if !UE_BUILD_SHIPPING
	if (Only.IsEmpty())
	{
		CommandTest();
	}
#endif
#if IS_PROGRAM
	if (Only.IsEmpty() || Only == TEXT("cache"))
	{
		CacheTests(Context);
	}
#endif //IS_PROGRAM
#if !UE_BUILD_SHIPPING
	if (Only.IsEmpty() || Only == TEXT("http"))
	{
		HttpTests(Context);
	}
#endif
	return 0;
}

////////////////////////////////////////////////////////////////////////////////
static FCommand TestCommand(
	TestCommandEntry,
	TEXT("Test"),
	TEXT("Run IAS tests"),
	{
		TArgument<FStringView>(TEXT("-Host"), TEXT("Host of the HTTP test server")),
		TArgument<FStringView>(TEXT("-Dir"), TEXT("Primary directory to use for cache tests")),
		TArgument<FStringView>(TEXT("-Only"), TEXT("Only run a particular test (http|cache)")),
		TArgument<uint32>(TEXT("-HttpSeed"), TEXT("Integer value to seed test HTTP server")),
	}
);

} // namespace Tool
} // namespace UE::IoStore
