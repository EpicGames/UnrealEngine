// Copyright Epic Games, Inc. All Rights Reserved.

#include "Upload.h"

#include "Command.h"
#include "Common.h"
#include "UploadQueue.h"

#include "HAL/FileManager.h"
#include "HAL/Platform.h"
#include "IO/IoChunkEncoding.h"
#include "IO/IoContainerHeader.h"
#include "IO/IoDispatcher.h"
#include "IO/IoStoreOnDemand.h"
#include "IO/Serialization/OnDemandContainerToc.h"
#include "Misc/Compression.h"
#include "Misc/ConfigCacheIni.h"
#include "Misc/KeyChainUtilities.h"
#include "Misc/Paths.h"
#include "Misc/PathViews.h"
#include "S3/S3Client.h"
#include "Serialization/CompactBinary.h"
#include "Serialization/CompactBinarySerialization.h"
#include "Serialization/CompactBinaryWriter.h"
#include "Serialization/MemoryReader.h"
#include "Serialization/LargeMemoryWriter.h"

namespace UE::IoStore::Tool { extern FArgumentSet S3Arguments; };

namespace UE::IoStore::Tool::Upload
{

using EOnDemandContainerEntryFlags = UE::IoStore::Serialization::EOnDemandContainerEntryFlags;

////////////////////////////////////////////////////////////////////////////////
FIoStatus LoadContainers(
	TConstArrayView<FString> FilePaths,
	const FEncryptionKeys& EncryptionKeys,
	TPagedArray<FContainerData>& OutContainers,
	bool bIgnoreContainerFlags)
{
	for (const FString& FilePath : FilePaths)
	{
		TUniquePtr<FIoStoreReader> MaybeReader = MakeUnique<FIoStoreReader>();
		if (FIoStatus Status = MaybeReader->Initialize(
			*FPaths::ChangeExtension(FilePath, TEXT("")),
			EncryptionKeys); !Status.IsOk())
		{
			return FIoStatusBuilder(EIoErrorCode::FileOpenFailed)
				<< TEXT("Failed to open container '") << FilePath << TEXT("'");
		}

		if (bIgnoreContainerFlags == false && 
			EnumHasAnyFlags(MaybeReader->GetContainerFlags(), EIoContainerFlags::OnDemand) == false)
		{
			continue;
		}

		FContainerData& Container	= OutContainers.Add_GetRef(FContainerData());
		Container.FilePath			= FilePath;
		Container.Name				= FPaths::GetBaseFilename(FilePath);
		Container.Reader			= MoveTemp(MaybeReader);
		FIoStoreReader& Reader		= *Container.Reader;

		Reader.EnumerateChunks([&Container](FIoStoreTocChunkInfo&& Info)
		{ 
			Container.ChunkInfos.Emplace(MoveTemp(Info));
			return true;
		});

		// Sort using the uncompressed chunk offset. This is different from the offset on disk.
		Container.ChunkInfos.Sort([](const FIoStoreTocChunkInfo& LHS, const FIoStoreTocChunkInfo& RHS)
		{
			return LHS.Offset < RHS.Offset;
		});

		{
			const bool bDecrypt = false;
			const FIoChunkId ChunkId = CreateContainerHeaderChunkId(Reader.GetContainerId());
			TIoStatusOr<FIoStoreCompressedReadResult> ReadResult = Reader.ReadCompressed(ChunkId, FIoReadOptions(), bDecrypt);
			if (ReadResult.IsOk())
			{
				Container.Header = ReadResult.ValueOrDie().IoBuffer;
			}
			else
			{
				const EIoErrorCode Err = ReadResult.Status().GetErrorCode();
				if (Err != EIoErrorCode::UnknownChunkID && Err != EIoErrorCode::NotFound)
				{
					return FIoStatusBuilder(EIoErrorCode::ReadError)
						<< TEXT("Failed to read container header '") << FilePath << TEXT("'");
				}
			}
		}

		{
			const FString UTocFilePath = FPaths::ChangeExtension(FilePath, TEXT(".utoc"));
			TUniquePtr<FArchive> Ar(IFileManager::Get().CreateFileReader(*UTocFilePath));

			if (!Ar || Ar->IsError())
			{
				return FIoStatusBuilder(EIoErrorCode::FileOpenFailed)
					<< TEXT("Failed to read container TOC '") << UTocFilePath << TEXT("'");
			}

			Container.UToc = FIoBuffer(Ar->TotalSize());
			Ar->Serialize(Container.UToc.GetData(), Ar->TotalSize());

			if (Ar->Close() == false)
			{
				return FIoStatusBuilder(EIoErrorCode::ReadError)
					<< TEXT("Failed to read container TOC '") << UTocFilePath << TEXT("'");
			}

			Container.UTocHash = FIoHash::HashBuffer(Container.UToc.GetView());
		}
	}

	return FIoStatus::Ok;
}

////////////////////////////////////////////////////////////////////////////////
FIoStatus LoadTagSets(
	const FString& FilePath,
	FTagSets& Out)
{
	IFileManager& Ifm = IFileManager::Get();

	if (FilePath.IsEmpty() || Ifm.FileExists(*FilePath) == false)
	{
		return FIoStatus(EIoErrorCode::NotFound);
	}

	TUniquePtr<FArchive> Ar(Ifm.CreateFileReader(*FilePath));
	if (!Ar || Ar->IsError())
	{
		return FIoStatus(
			EIoErrorCode::ReadError,
			FString::Printf(TEXT("Failed to open tag set file '%s'"), *FilePath));
	}

	TSharedRef<TJsonReader<UTF8CHAR>> JsonReader = TJsonReaderFactory<UTF8CHAR>::Create(Ar.Get());

	TSharedPtr<FJsonValue> JsonValue;
	if (!FJsonSerializer::Deserialize(*JsonReader, JsonValue))
	{
		return FIoStatus(
			EIoErrorCode::ReadError,
			FString::Printf(TEXT("Failed to read tag set file '%s'"), *FilePath));
	}

	TSharedPtr<FJsonObject> PackageSetsObject = JsonValue->AsObject();
	if (!PackageSetsObject)
	{
		return FIoStatus(
			EIoErrorCode::ReadError,
			FString::Printf(TEXT("Invalid JSON object in tag set file '%s'"), *FilePath));
	}

	for (const TPair<FJsonObject::FStringType, TSharedPtr<FJsonValue>>& Kv : PackageSetsObject->Values)
	{
		TArray<FString> PackageNames;
		if (!PackageSetsObject->TryGetStringArrayField(Kv.Key, PackageNames))
		{
			return FIoStatus(
				EIoErrorCode::ReadError,
				FString::Printf(TEXT("Invalid JSON value at '%s' in tag set file '%s'"),
					*Kv.Key, *FilePath));
		}

		Out.Add(FString(Kv.Key), MoveTemp(PackageNames));
	}

	return FIoStatus::Ok;
}

////////////////////////////////////////////////////////////////////////////////
FIoStatus CreateTagSets(
	const FTagSets& TagSets,
	FContainerData& Container)
{
	if (Container.Header.GetSize() == 0)
	{
		return FIoStatus(EIoErrorCode::InvalidParameter);
	}

	FIoContainerHeader ContainerHeader;
	{
		FMemoryReaderView Ar(Container.Header.GetView());
		Ar << ContainerHeader;
		if (Ar.Close() == false)
		{
			return FIoStatus(
				EIoErrorCode::ReadError,
				FString::Printf(TEXT("Failed to deserialize header chunk for container '%s'"),
					*Container.FilePath));
		}
	}

	for (const TPair<FString, TArray<FString>>& Kv : TagSets)
	{
		TConstArrayView<FString>	PackageNames = Kv.Value;
		TArray<uint32>				Indices;

		for (const FString& PackageName : PackageNames)
		{
			const FPackageId	PackageId = FPackageId::FromName(FName(PackageName));
			int32				Index = INDEX_NONE;

			if (ContainerHeader.PackageIds.Find(PackageId, Index))
			{
				check(Index >= 0);
				Indices.Add(uint32(Index));
			}
		}

		if (Indices.IsEmpty() == false)
		{
			Container.TagSets.Add(Kv.Key, MoveTemp(Indices));
		}
	}

	return FIoStatus::Ok;
}

////////////////////////////////////////////////////////////////////////////////
void GenerateTestChunks(
	FS3Client& Client,
	const FString& Bucket,
	const FString& BucketRelativePath,
	uint8 MaxExp)
{
	using namespace UE::IoStore::Serialization;

	// Generate chunks in the form of 1kib.iochunk - 32768kib.iochunk
	const int32 Exp = FMath::Clamp(int32(MaxExp), 1, 16);

	TStringBuilder<256> Key;
	for (int32 Shft = 0; Shft < Exp; ++Shft)
	{
		Key.Reset();
		const int32 ChunkId		= (1 << Shft);
		const int32 ChunkSize	= ChunkId << 10;
		Key << BucketRelativePath << TEXT("/") << ChunkId << TEXT("kib") << FOnDemandFileExt::Chunk;

		const FS3HeadObjectResponse HeadResponse = Client.HeadObject(FS3HeadObjectRequest{Bucket, Key.ToString()});
		if (HeadResponse.IsOk())
		{
			continue;
		}

		FIoBuffer Buffer(ChunkSize);
		ANSICHAR* BufferContent = reinterpret_cast<ANSICHAR*>(Buffer.GetData());
		FAnsiString SizeText = FAnsiString::Printf("%d", ChunkSize);
		for (int32 Idx = 0; Idx < Buffer.GetSize(); ++Idx)
		{
			BufferContent[Idx] = SizeText[Idx % SizeText.Len()];
		}

		const FS3PutObjectResponse Response = Client.TryPutObject(FS3PutObjectRequest
		{
			.BucketName = Bucket,
			.Key		= Key.ToString(),
			.ObjectData = Buffer.GetView()
		}); 

		if (Response.IsOk())
		{
			UE_LOGF(LogIoStoreOnDemand, Display, "Uploaded test chunk '%ls'", Key.ToString());
		}
		else
		{
			UE_LOGF(LogIoStoreOnDemand, Display, "Failed to upload test chunk '%ls'", Key.ToString());
		}
	}
}

////////////////////////////////////////////////////////////////////////////////
FIoStatus DownloadPartitionChunkHashes(
	FS3Client& Client,
	const FString& Bucket,
	const FString& BucketRelativePath,
	int32 MaxTocListCount,
	int32 MaxTocDownloadCount,
	TSet<FIoHash>& Out)
{
	if (Bucket.IsEmpty())
	{
		return FIoStatus(EIoErrorCode::InvalidParameter, TEXT("Invalid bucket name"));
	}

	TStringBuilder<256> Key;
	if (!BucketRelativePath.IsEmpty())
	{
		Key << BucketRelativePath << TEXT("/");
	}

	const int32 MaxKeys				= 1000; // AWS max value
	const int32 MaxPaginationCount	= FMath::Max(1, MaxTocListCount / MaxKeys);
	TArray<FS3Object> Objects;
	FString Marker;

	for (int32 Pagination = 0; Pagination < MaxPaginationCount; ++Pagination)
	{
		UE_LOGF(LogIoStoreOnDemand, Display, "List #%d/%d '%ls/%ls/%ls' TocCount=%d, MaxTocCount=%d",
			Pagination + 1, MaxPaginationCount, *Client.GetConfig().ServiceUrl,
			*Bucket, Key.ToString(), Objects.Num(), MaxTocListCount);

		FS3ListObjectResponse Response = Client.ListObjects(FS3ListObjectsRequest
		{
			.BucketName = Bucket,
			.Prefix		= Key.ToString(),
			.Delimiter	= TCHAR('/'),
			.MaxKeys	= MaxKeys,
			.Marker		= Marker
		});

		Marker = MoveTemp(Response.NextMarker);

		for (FS3Object& Object : Response.Objects)
		{
			if (Object.Key.EndsWith(UE::IoStore::Serialization::FOnDemandFileExt::PartitionToc))
			{
				Objects.Add(MoveTemp(Object));
			}
		}

		if (Response.IsOk() == false || Response.Objects.IsEmpty() || Response.bIsTruncated == false || Objects.Num() >= MaxTocListCount)
		{
			break;
		}
	}

	{
		// Just to be sure
		const int32 TotalObjectCount = Objects.Num();
		TSet<FS3Object> UniqueObjects(Objects);
		Objects = UniqueObjects.Array();

		if (Objects.Num() < TotalObjectCount)
		{
			UE_LOGF(LogIoStoreOnDemand, Display, "Removed %d non unique TOC objects", TotalObjectCount - Objects.Num());
		}
	}

	Objects.Sort([](const FS3Object& LHS, const FS3Object& RHS) { return LHS.LastModified > RHS.LastModified; });

	uint64 TotalExistingTocs		= 0;
	const int32 TocDownloadCount	= FMath::Min(Objects.Num(), MaxTocDownloadCount);

	if (TocDownloadCount == 0)
	{
		return FIoStatus::Ok;
	}

	UE_LOGF(LogIoStoreOnDemand, Display, "Fetching %d latest TOC file(s) ...", TocDownloadCount);

	for (int32 Idx = 0; Idx < TocDownloadCount; ++Idx)
	{
		const FS3Object& TocInfo = Objects[Idx];
		if (TocInfo.Key.EndsWith(UE::IoStore::Serialization::FOnDemandFileExt::PartitionToc) == false)
		{
			continue;
		}

		UE_LOGF(LogIoStoreOnDemand, Verbose, "Fetching TOC %d/%d '%ls/%ls/%ls', Size=%llu, LastModified=%ls",
			Idx + 1, TocDownloadCount, *Client.GetConfig().ServiceUrl, *Bucket,
			*TocInfo.Key, TocInfo.Size, *TocInfo.LastModifiedText);

		FS3GetObjectResponse TocResponse = Client.GetObject(FS3GetObjectRequest{ Bucket, TocInfo.Key });
		if (TocResponse.IsOk() == false)
		{
			UE_LOGF(LogIoStoreOnDemand, Warning, "Failed to fetch TOC '%ls/%ls/%ls'",
				*Client.GetConfig().ServiceUrl, *Bucket, *TocInfo.Key);
			continue;
		}

		FCbObjectView	TocView(TocResponse.GetBody().GetData());
		FCbArrayView	HashesView = TocView[UTF8TEXT("Hashes")].AsArrayView();

		for (FCbFieldView Item : HashesView)
		{
			const FIoHash& Hash = Item.AsHash();
			Out.Add(Hash);
		}

		TotalExistingTocs++;
	}

	return FIoStatus::Ok;
}

////////////////////////////////////////////////////////////////////////////////
FIoStatus UploadPartitionChunkHashes(
	FS3Client& Client,
	const FString& Bucket,
	const FString& BucketRelativePath,
	const FString& TargetPlatform,
	const FString& BuildVersion,
	TConstArrayView<FPartitionInfo> Partitions)
{
	FCbWriter Writer;
	Writer.BeginObject();
	{
		Writer.BeginObject(UTF8TEXT("Header"));
		{
			Writer << UTF8TEXT("Timestamp") << FDateTime::Now().ToUnixTimestamp();
			Writer << UTF8TEXT("TargetPlatform") << TargetPlatform;
			Writer << UTF8TEXT("BuildVersion") << BuildVersion;
		}
		Writer.EndObject();

		Writer.BeginArray(UTF8TEXT("Hashes"));
		for (const FPartitionInfo& Info : Partitions)
		{
			Writer.AddHash(Info.Hash);
		}
		Writer.EndArray();
	}
	Writer.EndObject();

	FLargeMemoryWriter Ar;
	Writer.Save(Ar);
	const FIoHash ObjHash = FIoHash::HashBuffer(Ar.GetView());

	TStringBuilder<256> Key;
	if (!BucketRelativePath.IsEmpty())
	{
		Key << BucketRelativePath << TEXT("/");
	}
	Key << ObjHash << UE::IoStore::Serialization::FOnDemandFileExt::PartitionToc;

	const FS3PutObjectResponse Response = Client.TryPutObject(FS3PutObjectRequest{ Bucket, Key.ToString(), Ar.GetView() });
	if (Response.IsOk() == false)
	{
		return FIoStatus(
			EIoErrorCode::WriteError,
			FString::Printf(TEXT("Failed to upload partition hash(s), reason: %s"),
				*Response.GetErrorStatus()));
	}

	return FIoStatus::Ok;
}

////////////////////////////////////////////////////////////////////////////////
static bool ShouldValidateChunk(
	const FChunkValidationSettings& ValidationSettings,
	FName CompressionFormat,
	uint64 RawChunkSize)
{
	static TMap<FName, bool> ValidatedFormats;

	if (ValidationSettings.bValidate == false)
	{
		return false;
	}

	if (!CompressionFormat.IsNone())
	{
		if (bool* bCachedResult = ValidatedFormats.Find(CompressionFormat))
		{
			if (*bCachedResult == false)
			{
				return false;
			}
		}
		else
		{
			// Note that if IsFormatValid returns false it will log a message about it. This is one of the
			// reasons we must make sure to only check once per format to avoid  log spam. The other reason
			// is that the check is fairly slow.
			bool bValid = FCompression::IsFormatValid(CompressionFormat);
			ValidatedFormats.Add(CompressionFormat, bValid);

			if (bValid == false)
			{
				UE_LOGF(LogIoStoreOnDemand, Display, "Skipping validation tests for chunks with the compression format '%ls'", *CompressionFormat.ToString())
				return false;
			}
		}
	}

	return RawChunkSize < ValidationSettings.MaxRawChunkSize;
}

////////////////////////////////////////////////////////////////////////////////
FIoStatus UploadContainer(
	FContainerData& Container,
	const FEncryptionKeys& EncryptionKeys,
	int32 MaxPartitionSize,
	UE::IoStore::Serialization::EOnDemandContainerEntryFlags ContainerFlags,
	const TSet<FIoHash>& ExistingChunks,
	FUploadChunkCallback&& Upload,
	UE::IoStore::Serialization::FOnDemandTocWriter& OutWriter,
	const FChunkValidationSettings& ValidationSettings)
{
	using namespace UE::IoStore::Serialization;

	check(Container.Reader.IsValid());
	FIoStoreReader& Reader = *Container.Reader;

	FIoBuffer	DecodedBuffer;
	FName		ContainerCompressionFormat	= NAME_None;
	int32		Partition					= INDEX_NONE;
	uint64		PartitionSize				= 0;

	auto EndPartition = [&Container, &ExistingChunks, &Upload, &OutWriter, &Partition, &PartitionSize]() -> FIoStatus
	{
		FIoHash			Hash;
		FIoBuffer		Chunk		= OutWriter.EndPartition(Partition, Hash);
		const FString	HashString	= LexToString(Hash);
		Partition					= INDEX_NONE;
		PartitionSize				= 0;

		Container.Stats.Partitions.Count++;
		Container.Stats.Partitions.Size += Chunk.GetSize();

		if (ExistingChunks.Contains(Hash))
		{
			return FIoStatus::Ok;
		}

		if (FIoStatus Status = Upload(Chunk, Hash); !Status.IsOk())
		{
			return Status;
		}

		Container.UploadedPartitions.Add(FPartitionInfo
		{
			.Hash = Hash,
			.Size = Chunk.GetSize()
		});

		Container.Stats.UploadedPartitions.Count++;
		Container.Stats.UploadedPartitions.Size += Chunk.GetSize();

		return FIoStatus::Ok;
	};

	OutWriter.BeginContainer(
		Container.Name,	
		Reader.GetContainerId(),
		Container.UTocHash,
		Reader.GetEncryptionKeyGuid(),
		Reader.GetCompressionBlockSize(),
		uint32(Reader.GetContainerFlags()),
		ContainerFlags,
		Container.Header);

	for (const FIoStoreTocChunkInfo& ChunkInfo : Container.ChunkInfos)
	{
		const bool bDecrypt = false;
		TIoStatusOr<FIoStoreCompressedReadResult> ReadStatus = Reader.ReadCompressed(
			ChunkInfo.Id,
			FIoReadOptions(),
			bDecrypt);

		if (!ReadStatus.IsOk())
		{
			return ReadStatus.Status();
		}

		FIoStoreCompressedReadResult	ReadResult = ReadStatus.ConsumeValueOrDie();
		FMemoryView						EncodedBlocks = ReadResult.IoBuffer.GetView();
		TArray<uint32>					BlockSizes;
		TArray<FIoBlockHash>			BlockHashes;
		uint64							RawChunkSize = 0;
		uint64							EncodedChunkSize = 0;
		FName							CompressionFormat = NAME_None;

		for (const FIoStoreCompressedBlockInfo& BlockInfo : ReadResult.Blocks)
		{
			check(Align(BlockInfo.CompressedSize, FAES::AESBlockSize) == BlockInfo.AlignedSize);
			const uint64 EncodedBlockSize = BlockInfo.AlignedSize;
			BlockSizes.Add(uint32(BlockInfo.CompressedSize));
			
			FMemoryView EncodedBlock = EncodedBlocks.Left(EncodedBlockSize);
			EncodedBlocks += EncodedBlock.GetSize();
			BlockHashes.Add(FIoChunkEncoding::HashBlock(EncodedBlock));

			EncodedChunkSize += EncodedBlockSize;
			RawChunkSize += BlockInfo.UncompressedSize;

			if (CompressionFormat.IsNone() && BlockInfo.CompressionMethod != NAME_None)
			{
				CompressionFormat = BlockInfo.CompressionMethod;
			}
		}

		// Assume only a single compression format
		if (ContainerCompressionFormat.IsNone())
		{
			ContainerCompressionFormat = CompressionFormat;
		}
		else if (!CompressionFormat.IsNone() && CompressionFormat != ContainerCompressionFormat)
		{
			return FIoStatus(EIoErrorCode::InvalidParameter, TEXT("Container contains multiple compression format(s)"));
		}

		if (EncodedChunkSize != ReadResult.IoBuffer.GetSize())
		{
			return FIoStatus(EIoErrorCode::ReadError, TEXT("Encoded chunk size does not match buffer"));
		}

		// At runtime we are limited to MAX_uint32 for chunk lengths to save space and because anything larger than that
		// is not reasonable to load via IoStoreOnDemand anyway. So we need to check for this now and fail the upload if
		// there is a chunk that will fail at runtime.
		// Note that EncodedSize should always be <= RawChunkSize but test both to be safe.
		if (RawChunkSize > MAX_uint32 || EncodedChunkSize > MAX_uint32)
		{
			return FIoStatus(EIoErrorCode::InvalidParameter, WriteToString<512>(*ChunkInfo.FileName, TEXT(": Chunk size should not exceed MAX_uint32")));
		}

		if (ShouldValidateChunk(ValidationSettings, CompressionFormat, RawChunkSize))
		{
			FIoChunkDecodingParams Params;
			Params.CompressionFormat	= CompressionFormat;
			Params.TotalRawSize			= RawChunkSize;
			Params.EncodedBlockSize		= BlockSizes; 
			Params.BlockHash			= BlockHashes;

			if (EnumHasAnyFlags(Reader.GetContainerFlags(), EIoContainerFlags::Encrypted))
			{
				const FAES::FAESKey* EncryptionKey = EncryptionKeys.Find(Reader.GetEncryptionKeyGuid());
				if (EncryptionKey == nullptr)
				{
					return FIoStatus(EIoErrorCode::InvalidParameter, TEXT("Encryption key not found"));
				}

				Params.EncryptionKey = FMemoryView(EncryptionKey->Key, EncryptionKey->KeySize);
			}

			if (DecodedBuffer.GetSize() < RawChunkSize)
			{
				DecodedBuffer = FIoBuffer(FMath::RoundUpToPowerOfTwo64(RawChunkSize));
			}

			FMemoryView			Src = ReadResult.IoBuffer.GetView();
			FMutableMemoryView	Dst = DecodedBuffer.GetMutableView().Left(RawChunkSize);

			if (FIoChunkEncoding::Decode(Params, Src, Dst, EIoDecodeFlags::None) == false)
			{
				return FIoStatus(
					EIoErrorCode::CompressionError,
					FString::Printf(TEXT("Failed to decode chunk, Container='%s', ChunkId='%s'"),
						*Container.Name, *LexToString(ChunkInfo.Id)));
			}
		}

		// Check if the chunk fits in the current partition. If the partition size is zero
		// there will be a one to one mapping beteween partitions and chunks and that is supported.
		if (PartitionSize > 0 && (PartitionSize + EncodedChunkSize >= MaxPartitionSize))
		{
			if (FIoStatus Status = EndPartition(); !Status.IsOk())
			{
				return Status;
			}
		}

		if (Partition == INDEX_NONE)
		{
			Partition = OutWriter.BeginPartition();
			check(Partition >= 0);
			check(PartitionSize == 0);
		}

		PartitionSize = OutWriter.AddChunk(
			Partition,
			ChunkInfo.Id,
			ReadResult.IoBuffer,
			BlockSizes,
			BlockHashes,
			RawChunkSize);

		Container.Stats.Chunks.Count++;
		Container.Stats.Chunks.Size += EncodedChunkSize;
	}

	// Flush remaining chunks into the current super chunk
	if (Partition != INDEX_NONE)
	{
		if (FIoStatus Status = EndPartition(); !Status.IsOk())
		{
			return Status;
		}
	}

	check(Partition == INDEX_NONE);
	check(PartitionSize == 0);

	for (const TPair<FString, TArray<uint32>>& Kv : Container.TagSets)
	{
		check(!Kv.Key.IsEmpty());
		check(!Kv.Value.IsEmpty());
		OutWriter.AddTagSet(Kv.Key, Kv.Value);
	}

	OutWriter.EndContainer(ContainerCompressionFormat);

	return FIoStatus::Ok;
}

////////////////////////////////////////////////////////////////////////////////
struct FUploadParams
{
	FString ServiceUrl;
	FString Bucket;
	FString BucketPrefix;
	FString Region; 
	FString AccessKey;
	FString SecretKey;
	FString SessionToken;
	FString CredentialsFile;
	FString CredentialsFileKeyName;
	FString BuildVersion;
	FString TargetPlatform;
	FString HostGroupName;
	FString GlobalContainerPath;
	FString TagSetFile;
	int32 MaxConcurrentUploads = 16;
	int32 MaxTocListCount = 10000;
	int32 MaxTocDownloadCount = 100;
	int32 MaxPartitionSize = 0;
	EOnDemandContainerEntryFlags ContainerFlags = EOnDemandContainerEntryFlags::None;

	bool bDeleteContainerFiles = true;
	bool bDeletePakFiles = true;
	bool bPerContainerTocs = false;
	bool bIgnoreContainerFlags = false;
	bool bValidateChunks = false;

	FIoStatus Validate() const;
};

FIoStatus FUploadParams::Validate() const
{
	if (!AccessKey.IsEmpty() && SecretKey.IsEmpty())
	{
		return FIoStatus(EIoErrorCode::InvalidParameter, TEXT("Invalid secret key"));
	}
	else if (AccessKey.IsEmpty() && !SecretKey.IsEmpty())
	{
		return FIoStatus(EIoErrorCode::InvalidParameter, TEXT("Invalid access key"));
	}

	if (!CredentialsFile.IsEmpty() && CredentialsFileKeyName.IsEmpty())
	{
		return FIoStatus(EIoErrorCode::InvalidParameter, TEXT("Invalid credential file key name"));
	}

	if (ServiceUrl.IsEmpty() && Region.IsEmpty())
	{
		return FIoStatus(EIoErrorCode::InvalidParameter, TEXT("Service URL or AWS region needs to be specified"));
	}

	return FIoStatus::Ok;
}

////////////////////////////////////////////////////////////////////////////////
static FUploadParams BuildUploadParams(const FContext& Context)
{
	FUploadParams Ret;

	Ret.ServiceUrl				= Context.Get<FStringView>(TEXT("-ServiceUrl"),				Ret.ServiceUrl);
	Ret.Bucket					= Context.Get<FStringView>(TEXT("-Bucket"),					Ret.Bucket);
	Ret.BucketPrefix			= Context.Get<FStringView>(TEXT("-BucketPrefix"),			Ret.BucketPrefix);
	Ret.Region					= Context.Get<FStringView>(TEXT("-Region"),					Ret.Region);
	Ret.AccessKey				= Context.Get<FStringView>(TEXT("-AccessKey"),				Ret.AccessKey);
	Ret.SecretKey				= Context.Get<FStringView>(TEXT("-SecretKey"),				Ret.SecretKey);
	Ret.SessionToken			= Context.Get<FStringView>(TEXT("-SessionToken"),			Ret.SessionToken);
	Ret.CredentialsFile			= Context.Get<FStringView>(TEXT("-CredentialsFile"),		Ret.CredentialsFile);
	Ret.CredentialsFileKeyName	= Context.Get<FStringView>(TEXT("-CredentialsFileKeyName"),	Ret.CredentialsFileKeyName);
	Ret.BuildVersion			= Context.Get<FStringView>(TEXT("-BuildVersion"),			Ret.BuildVersion);
	Ret.TargetPlatform			= Context.Get<FStringView>(TEXT("-TargetPlatform"),			Ret.TargetPlatform);
	Ret.HostGroupName			= Context.Get<FStringView>(TEXT("-HostGroupName"),			Ret.HostGroupName);
	Ret.GlobalContainerPath		= Context.Get<FStringView>(TEXT("-GlobalContainerPath"),	Ret.GlobalContainerPath);
	Ret.TagSetFile				= Context.Get<FStringView>(TEXT("-TagSetFile"),				Ret.TagSetFile);
	Ret.bPerContainerTocs		= Context.Get<bool>(TEXT("-PerContainerTocs"),				Ret.bPerContainerTocs);
	Ret.MaxConcurrentUploads	= Context.Get<int32>(TEXT("-MaxConcurrentUploads"),			FUploadSettings::MaxConcurrentUploads);
	Ret.MaxTocListCount			= Context.Get<int32>(TEXT("-MaxTocListCount"),				FUploadSettings::MaxTocListCount);
	Ret.MaxTocDownloadCount		= Context.Get<int32>(TEXT("-MaxTocDownloadCount"),			FUploadSettings::MaxTocDownloadCount);
	Ret.MaxPartitionSize		= Context.Get<int32>(TEXT("-MaxPartitionSize"),				FUploadSettings::DefaultPartitionSize);

	const bool bStreamOnDemand	= Context.Get<bool>(TEXT("-StreamOnDemand"),				false);
	const bool bInstallOnDemand	= Context.Get<bool>(TEXT("-InstallOnDemand"),				false);

	if (bStreamOnDemand)
	{
		Ret.ContainerFlags = EOnDemandContainerEntryFlags::StreamOnDemand;
	}
	else if (bInstallOnDemand)
	{
		Ret.ContainerFlags = EOnDemandContainerEntryFlags::InstallOnDemand;
	}
	else
	{
		EOnDemandContainerEntryFlags FallbackFlags = EOnDemandContainerEntryFlags::StreamOnDemand;
		UE_LOGF(LogIoStoreOnDemand, Warning, "No TOC flags was specified, falling back to '%ls'", *LexToString(FallbackFlags));
		Ret.ContainerFlags = FallbackFlags;
	}

	Ret.bDeleteContainerFiles	= !Context.Get<bool>(TEXT("-KeepContainerFiles"),			!Ret.bDeleteContainerFiles);
	Ret.bDeletePakFiles			= !Context.Get<bool>(TEXT("-KeepPakFiles"),					!Ret.bDeletePakFiles);
	Ret.bIgnoreContainerFlags	= Context.Get<bool>(TEXT("-IgnoreContainerFlags"),			Ret.bIgnoreContainerFlags);
	Ret.bValidateChunks			= Context.Get<bool>(TEXT("-ValidateChunks"),				false);

	if (FIoStatus Validation = Ret.Validate(); !Validation.IsOk())
	{
		FString Reason = Validation.ToString();
		Context.Abort(*Reason);
	}

	Ret.MaxTocListCount		= FMath::Clamp(Ret.MaxTocListCount, FUploadSettings::MinTocListCount, FUploadSettings::MaxTocListCount);
	Ret.MaxTocDownloadCount = FMath::Clamp(Ret.MaxTocDownloadCount, 1, FUploadSettings::MaxTocDownloadCount);
	Ret.MaxPartitionSize	= FMath::Clamp(Ret.MaxPartitionSize, 0, FUploadSettings::MaxPartitionSize);

	// Make sure this is a directory path
	if (FPaths::GetExtension(Ret.GlobalContainerPath).IsEmpty() == false)
	{
		Ret.GlobalContainerPath	= FPaths::GetPath(Ret.GlobalContainerPath);
	}

	bool bPerContainerTocsConfigValue = false;
	if (GConfig->GetBool(TEXT("Ias"), TEXT("CreatePerContainerTocs"), bPerContainerTocsConfigValue, GEngineIni))
	{
		if (bPerContainerTocsConfigValue)
		{
			Ret.bPerContainerTocs = true;
		}
	}

	return Ret;
}

////////////////////////////////////////////////////////////////////////////////
static void GetTocPath(
	FStringView ServiceUrl,
	FStringView Bucket,
	FStringView BucketPrefix,
	FString& OutServiceUrl,
	FString& OutTocPath)
{
	// The configuration file should specify a service URL without any trailing
	// host path, i.e. http://{host:port}/{host-path}. Add the trailing path
	// to the TOC path to form the complete path the TOC from the host, i.e
	// TocPath={host-path}/{bucket}/{bucket-prefix}/{toc-hash}.iochunktoc
	
	if (BucketPrefix.StartsWith(TEXT("/")))
	{
		BucketPrefix.RemovePrefix(1);
	}
	if (BucketPrefix.EndsWith(TEXT("/")))
	{
		BucketPrefix.RemoveSuffix(1);
	}

	if (ServiceUrl.IsEmpty())
	{
		// If the service URL is empty we assume uploading to AWS S3 using the Region parameter
		// and that we don't need to prefix with the bucket name.
		OutServiceUrl.Empty();
		OutTocPath = BucketPrefix;
	}
	else
	{
		FStringView HostSuffix;
		int32 Idx = INDEX_NONE;
		ensure(ServiceUrl.FindChar(':', Idx));
		const int32 SchemeEnd = Idx + 3;

		Idx = INDEX_NONE;
		if (ServiceUrl.RightChop(SchemeEnd).FindChar('/', Idx))
		{
			OutServiceUrl = ServiceUrl.Left(SchemeEnd + Idx);
			HostSuffix = ServiceUrl.RightChop(OutServiceUrl.Len() + 1);
			if (HostSuffix.EndsWith(TEXT("/")))
			{
				HostSuffix.RemoveSuffix(1);
			}
		}
		else
		{
			OutServiceUrl = ServiceUrl;
		}

		TStringBuilder<256> Sb;
		if (!HostSuffix.IsEmpty())
		{
			Sb << HostSuffix << TEXT("/");
		}
		Sb << Bucket << TEXT("/") << BucketPrefix;
		OutTocPath = Sb;
	}
}

////////////////////////////////////////////////////////////////////////////////
FIoStatus SaveContainerToc(
	UE::IoStore::Serialization::FOnDemandTocWriter& TocWriter,
	const FString& FilePath,
	const FUploadParams& UploadParams)
{
	FString ServiceUrl;
	FString ChunksDirectory;
	GetTocPath(UploadParams.ServiceUrl, UploadParams.Bucket, UploadParams.BucketPrefix, ServiceUrl, ChunksDirectory);

	TocWriter.SetMetadata(UploadParams.BuildVersion, UploadParams.TargetPlatform);
	TocWriter.SetChunksDirectory(ChunksDirectory);
	TocWriter.SetHostGroup(UploadParams.HostGroupName);

	if (TIoStatusOr<uint64> Status = TocWriter.Write(FilePath); Status.IsOk())
	{
		UE_LOGF(LogIoStoreOnDemand, Display, "Saved on-demand TOC file '%ls' (%.2lf KiB)",
			*FilePath, double(Status.ConsumeValueOrDie()) / 1024.0);

		return FIoStatus::Ok;
	}
	else
	{
		return Status.Status();
	}
}

////////////////////////////////////////////////////////////////////////////////
static FIoStatus UploadContainerFiles(
	const FUploadParams& UploadParams,
	TConstArrayView<FString> ContainerFiles,
	const FKeyChain& KeyChain,
	TArray<FString>& OutFilesToDelete)
{
	using namespace UE::IoStore::Serialization;
	
	const bool bGlobalToc = UploadParams.GlobalContainerPath.IsEmpty() == false && UploadParams.bPerContainerTocs == false;

	TMap<FGuid, FAES::FAESKey> EncryptionKeys;
	for (const TPair<FGuid, FNamedAESKey>& KeyPair: KeyChain.GetEncryptionKeys())
	{
		EncryptionKeys.Add(KeyPair.Key, KeyPair.Value.Key);
	}

	FS3ClientConfig Config;
	Config.ServiceUrl = UploadParams.ServiceUrl;
	Config.Region = UploadParams.Region;
	
	FS3ClientCredentials Credentials;
	if (UploadParams.CredentialsFile.IsEmpty() == false)
	{
		UE_LOGF(LogIoStoreOnDemand, Display, "Loading credentials file '%ls'", *UploadParams.CredentialsFile);
		FS3CredentialsProfileStore CredentialsStore = FS3CredentialsProfileStore::FromFile(UploadParams.CredentialsFile);
		if (CredentialsStore.TryGetCredentials(UploadParams.CredentialsFileKeyName, Credentials) == false)
		{
			return FIoStatus(EIoErrorCode::InvalidParameter, TEXT("Failed to find valid credentials in credentials file"));
		}
		else
		{
			UE_LOGF(LogIoStoreOnDemand, Display, "Found credentials for '%ls'", *UploadParams.CredentialsFileKeyName);
		}
	}
	else
	{
		Credentials = FS3ClientCredentials(UploadParams.AccessKey, UploadParams.SecretKey, UploadParams.SessionToken);
	}

	FS3Client Client(Config, Credentials);
	FUploadQueue UploadQueue(Client, UploadParams.Bucket, UploadParams.MaxConcurrentUploads);

	FString ChunksRelativePath = UploadParams.BucketPrefix.IsEmpty()
		? TEXT("Chunks")
		: FString::Printf(TEXT("%s/Chunks"), *UploadParams.BucketPrefix);
	ChunksRelativePath.ToLowerInline();

	TPagedArray<FContainerData> Containers;
	if (FIoStatus Status = LoadContainers(
		ContainerFiles,
		EncryptionKeys,
		Containers,
		UploadParams.bIgnoreContainerFlags); !Status.IsOk())
	{
		return Status;
	}

	if (Containers.IsEmpty())
	{
		return FIoStatus(EIoErrorCode::InvalidParameter, TEXT("No container file(s) marked as on demand"));
	}

	FTagSets TagSets;
	if (FIoStatus Status = LoadTagSets(UploadParams.TagSetFile, TagSets); Status.IsOk())
	{
		UE_LOGF(LogIoStoreOnDemand, Display, "Creating tag set(s) from '%ls'", *UploadParams.TagSetFile);
		for (FContainerData& Container : Containers)
		{
			// Can only create tag sets for containers with package information
			if (Container.Header.GetSize() > 0)
			{
				if (Status = CreateTagSets(TagSets, Container); !Status.IsOk())
				{
					return Status;
				}
			}
		}
	}
	else if (Status.GetErrorCode() != EIoErrorCode::NotFound)
	{
		return Status;
	}

	TSet<FIoHash> ExistingChunks;
	if (FIoStatus Status = DownloadPartitionChunkHashes(
		Client,
		UploadParams.Bucket,
		UploadParams.BucketPrefix,
		UploadParams.MaxTocListCount,
		UploadParams.MaxTocDownloadCount,
		ExistingChunks); !Status.IsOk())
	{
		return Status;
	}

	FOnDemandTocWriter GlobalToc;
	for (FContainerData& Container : Containers)
	{
		auto Upload = [&UploadQueue, &ChunksRelativePath](FIoBuffer Chunk, const FIoHash& Hash) -> FIoStatus
		{
			const FString HashString = LexToString(Hash);

			TStringBuilder<256> Key;
			Key << ChunksRelativePath
				<< TEXT("/") << HashString.Left(2)
				<< TEXT("/") << HashString
				<< UE::IoStore::Serialization::FOnDemandFileExt::Partition;

			if (UploadQueue.Enqueue(Key.ToView(), Chunk) == false)
			{
				return FIoStatus(EIoErrorCode::WriteError);
			}

			return FIoStatus::Ok;
		};

		FChunkValidationSettings ValidationSettings
		{
			.bValidate = UploadParams.bValidateChunks 
		};

		FOnDemandTocWriter ContainerToc;
		FOnDemandTocWriter& Writer = bGlobalToc ? GlobalToc : ContainerToc;
		if (FIoStatus Status = UploadContainer(
			Container,
			EncryptionKeys,
			UploadParams.MaxPartitionSize,
			UploadParams.ContainerFlags,
			ExistingChunks,
			MoveTemp(Upload),
			Writer,
			ValidationSettings); !Status.IsOk())
		{
			return Status;
		}

		if (UploadParams.bDeleteContainerFiles)
		{
			OutFilesToDelete.Add(Container.FilePath);
			Container.Reader->GetContainerFilePaths(OutFilesToDelete);

			if (UploadParams.bDeletePakFiles)
			{
				OutFilesToDelete.Add(FPaths::ChangeExtension(Container.FilePath, TEXT(".pak")));
				OutFilesToDelete.Add(FPaths::ChangeExtension(Container.FilePath, TEXT(".sig")));
			}
		}

		if (bGlobalToc == false)
		{
			const FString FilePath = FPaths::ChangeExtension(Container.FilePath, FOnDemandFileExt::Toc);
			if (FIoStatus Status = SaveContainerToc(ContainerToc, FilePath, UploadParams); !Status.IsOk())
			{
				return Status;
			}
		}
	}

	if (bGlobalToc)
	{
		TStringBuilder<512> Sb;
		FPathViews::Append(Sb, UploadParams.GlobalContainerPath, FString(TEXT("global")) + FOnDemandFileExt::Toc);
		if (FIoStatus Status = SaveContainerToc(GlobalToc, Sb.ToString(), UploadParams); !Status.IsOk())
		{
			return Status;
		}
	}

	if (!UploadQueue.Flush())
	{
		return FIoStatus(EIoErrorCode::WriteError, TEXT("Failed to upload chunk(s)"));
	}

	GenerateTestChunks(Client, UploadParams.Bucket, ChunksRelativePath);

	TStringBuilder<256> BucketRelativePath;
	if (!UploadParams.BucketPrefix.IsEmpty())
	{
		BucketRelativePath << UploadParams.BucketPrefix;
	}

	for (const FContainerData& Container : Containers)
	{
		if (FIoStatus Status = UploadPartitionChunkHashes(
			Client,
			UploadParams.Bucket,
			BucketRelativePath.ToString(),
			UploadParams.TargetPlatform,
			UploadParams.BuildVersion,
			Container.UploadedPartitions); !Status.IsOk())
		{
			return Status;
		}
	}

	// Upload the original .utoc file(s)
	{
		for (const FContainerData& Container : Containers)
		{
			TStringBuilder<256> Key;
			if (!UploadParams.BucketPrefix.IsEmpty())
			{
				Key << UploadParams.BucketPrefix << TEXT("/");
			}
			Key << Container.UTocHash << TEXT(".utoc");

			const FS3PutObjectResponse Response = Client.TryPutObject(FS3PutObjectRequest{ UploadParams.Bucket, Key.ToString(), Container.UToc.GetView() });
			if (Response.IsOk())
			{
				UE_LOGF(LogIoStoreOnDemand, Display, "Uploaded file TOC '%ls/%ls/%ls'",
					*Client.GetConfig().ServiceUrl, *UploadParams.Bucket, Key.ToString());
			}
			else
			{
				return FIoStatus(
					EIoErrorCode::WriteError,
					FString::Printf(TEXT("Failed to upload container file TOC, reason: %s"),
						*Response.GetErrorStatus()));
			}
		}
	}

	UE_LOGF(LogIoStoreOnDemand, Display, "");
	UE_LOGF(LogIoStoreOnDemand, Display, "---------------------------------------------------- Upload Summary ----------------------------------------------------");
	UE_LOGF(LogIoStoreOnDemand, Display, "%-20ls: %ls", TEXT("Service URL"), *UploadParams.ServiceUrl);
	UE_LOGF(LogIoStoreOnDemand, Display, "%-20ls: %ls", TEXT("Bucket"), *UploadParams.Bucket);
	UE_LOGF(LogIoStoreOnDemand, Display, "%-20ls: %ls", TEXT("BucketPrefix"), *UploadParams.BucketPrefix);
	UE_LOGF(LogIoStoreOnDemand, Display, "%-20ls: %ls", TEXT("TargetPlatform"), *UploadParams.TargetPlatform);
	UE_LOGF(LogIoStoreOnDemand, Display, "%-20ls: %ls", TEXT("BuildVersion"), *UploadParams.BuildVersion);
	UE_LOGF(LogIoStoreOnDemand, Display, "%-20ls: %ls", TEXT("HostGroupName"), *UploadParams.HostGroupName);
	UE_LOGF(LogIoStoreOnDemand, Display, "%-20ls: %ls", TEXT("Flags"), *LexToString(UploadParams.ContainerFlags));
	UE_LOGF(LogIoStoreOnDemand, Display, "%-20ls: %.2lf MiB", TEXT("PartitionSize"), double(UploadParams.MaxPartitionSize) / 1024.0 / 1024.0);
	UE_LOGF(LogIoStoreOnDemand, Display, "");

	UE_LOGF(LogIoStoreOnDemand, Display, "%-40ls %15ls %15ls %15ls %15ls %15ls",
		TEXT("Container"), TEXT("Size (MiB)"), TEXT("Chunk(s)"), TEXT("Partition(s)"), TEXT("Uploaded"), TEXT("Uploaded (MiB)"));
	UE_LOGF(LogIoStoreOnDemand, Display, "------------------------------------------------------------------------------------------------------------------------");

	FContainerStats TotalStats;
	for (const FContainerData& Container : Containers)
	{
		const FContainerStats& Stats = Container.Stats;
		UE_LOGF(LogIoStoreOnDemand, Display, "%-40ls %15.2lf %15llu %15llu %15llu %15.2lf",
			*Container.Name, double(Stats.Chunks.Size) / 1024.0 / 1024.0, Stats.Chunks.Count,
			Stats.Partitions.Count, Stats.UploadedPartitions.Count, double(Stats.UploadedPartitions.Size) / 1024.0 / 1024.0);
		
		TotalStats.Chunks				= TotalStats.Chunks + Stats.Chunks;
		TotalStats.Partitions			= TotalStats.Partitions + Stats.Partitions;
		TotalStats.UploadedPartitions	= TotalStats.UploadedPartitions + Stats.UploadedPartitions;
	}

	UE_LOGF(LogIoStoreOnDemand, Display, "------------------------------------------------------------------------------------------------------------------------");
	UE_LOGF(LogIoStoreOnDemand, Display, "%-40ls %15.2lf %15llu %15llu %15llu %15.2lf",
		TEXT("Total"), double(TotalStats.Chunks.Size) / 1024.0 / 1024.0, TotalStats.Chunks.Count,
		TotalStats.Partitions.Count, TotalStats.UploadedPartitions.Count,
		double(TotalStats.UploadedPartitions.Size) / 1024.0 / 1024.0);
	UE_LOGF(LogIoStoreOnDemand, Display, "");

	return FIoStatus::Ok;
}

////////////////////////////////////////////////////////////////////////////////
static TArray<FString> GlobContainers(const FContext& Context)
{
	const TCHAR* GlobPattern = Context.Get<FStringView>(TEXT("ContainerGlob")).GetData();

	TArray<FString> Ret;

	if (IFileManager::Get().FileExists(GlobPattern))
	{
		Ret.Add(GlobPattern);
	}
	else if (IFileManager::Get().DirectoryExists(GlobPattern))
	{
		FString Directory = GlobPattern;
		FPaths::NormalizeDirectoryName(Directory);

		TArray<FString> FoundContainerFiles;
		IFileManager::Get().FindFiles(FoundContainerFiles, *(Directory / TEXT("*.utoc")), true, false);

		for (const FString& Filename : FoundContainerFiles)
		{
			Ret.Emplace(Directory / Filename);
		}
	}
	else
	{
		FString Directory = FPaths::GetPath(GlobPattern);
		FPaths::NormalizeDirectoryName(Directory);

		TArray<FString> FoundContainerFiles;
		IFileManager::Get().FindFiles(FoundContainerFiles, GlobPattern, true, false);

		for (const FString& Filename : FoundContainerFiles)
		{
			Ret.Emplace(Directory / Filename);
		}
	}

	return Ret;
}

////////////////////////////////////////////////////////////////////////////////
static int32 UploadCommandEntry(const FContext& Context)
{
	TArray<FString> Containers = GlobContainers(Context);
	FKeyChain KeyChain = Common::LoadCryptoKeys(Context);
	FUploadParams Params = BuildUploadParams(Context);

	TArray<FString> FilesToDelete;
	const FIoStatus Status = UploadContainerFiles(Params, Containers, KeyChain, FilesToDelete);
	if (!Status.IsOk())
	{
		FString Reason = Status.ToString();
		Context.Abort(*Reason);
	}

	// Delete the input file(s)
	for (const FString& Path : FilesToDelete)
	{
		if (IFileManager::Get().FileExists(*Path))
		{
			UE_LOGF(LogIoStoreOnDemand, Display, "Deleting '%ls'", *Path);
			if (IFileManager::Get().Delete(*Path) == false)
			{
				FString Reason = FString::Printf(TEXT("Failed to delete '%s'"), *Path);
				Context.Abort(*Reason);
			}
		}
	}

	return 0;
}

////////////////////////////////////////////////////////////////////////////////
static FCommand UploadCommand(
	UploadCommandEntry,
	TEXT("Upload"),
	TEXT("Used to upload IoStore containers to the cloud and convert to on-demand"),
	{
		TArgument<FStringView>(TEXT("ContainerGlob"),			TEXT("Path globbed to discover input containers")),
		TArgument<FStringView>(TEXT("-CryptoKeys"),				TEXT("JSON-format keyring for input containers")),
		TArgument<FStringView>(TEXT("-BuildVersion"),			TEXT("Optional build version to embed it TOC")),
		TArgument<FStringView>(TEXT("-TargetPlatform"),			TEXT("If given, embedded in the output TOC")),
		TArgument<FStringView>(TEXT("-ConfigFilePath"),			TEXT("Path to the config file to write runtime parameters to")),
		TArgument<FStringView>(TEXT("-BucketPrefix"),			TEXT("Path to prefix to bucket objects")),
		TArgument<FStringView>(TEXT("-HostGroupName"),			TEXT("Host group name or URL")),
		TArgument<FStringView>(TEXT("-GlobalContainerPath"),	TEXT("Directory path to the content PAK folder")),
		TArgument<FStringView>(TEXT("-TagSetFile"),				TEXT("Tag set JSON file.")),
		TArgument<bool>(TEXT("-KeepContainerFiles"),			TEXT("Do not delete container files after upload")),
		TArgument<bool>(TEXT("-KeepPakFiles"),					TEXT("Do not delete the springboard pak files")),
		TArgument<bool>(TEXT("-PerContainerTocs"),				TEXT("Whether to generate TOC's for each container file(s)")),
		TArgument<bool>(TEXT("-IgnoreContainerFlags"),			TEXT("Whether to ignore the OnDemand container flag")),
		TArgument<bool>(TEXT("-StreamOnDemand"),				TEXT("Set the content to be streamed on-demand")),
		TArgument<bool>(TEXT("-InstallOnDemand"),				TEXT("Set the content to be installed on-demand")),
		TArgument<int32>(TEXT("-MaxConcurrentUploads"),			TEXT("Number of simultaneous uploads")),
		TArgument<int32>(TEXT("-MaxTocListCount"),				TEXT("Maximum number of TOC file(s) to list from the bucket")),
		TArgument<int32>(TEXT("-MaxTocDownloadCount"),			TEXT("Maximum number of TOC file(s) to download")),
		TArgument<int32>(TEXT("-MaxPartitionSize"),				TEXT("Max partition size in bytes.")),
		TArgument<bool>(TEXT("-ValidateChunks"),				TEXT("Runs additional validation on chunks after they are read from the container and before they are written out")),
		S3Arguments,
	}
);

} // namespace UE::IoStore::Tool::Upload

