// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "Containers/Map.h"
#include "Containers/PagedArray.h"
#include "Containers/UnrealString.h"
#include "IO/IoBuffer.h"
#include "IO/IoHash.h"
#include "IO/IoStatus.h"
#include "Misc/AES.h"
#include "Templates/FunctionFwd.h"

struct FIoStoreTocChunkInfo;
class FIoStoreReader;

namespace UE { class FS3Client; }
namespace UE::IoStore::Serialization { class FOnDemandTocWriter; }
namespace UE::IoStore::Serialization { enum class EOnDemandContainerEntryFlags : uint32; }

namespace UE::IoStore::Tool::Upload
{

////////////////////////////////////////////////////////////////////////////////
using FTagSets			= TMap<FString, TArray<FString>>;
using FEncryptionKeys	= TMap<FGuid, FAES::FAESKey>;

////////////////////////////////////////////////////////////////////////////////
struct FUploadSettings
{
	static constexpr int32 MaxConcurrentUploads	= 16;
	static constexpr int32 MinTocListCount		= 1000;
	static constexpr int32 MaxTocListCount		= 10000;
	static constexpr int32 MaxTocDownloadCount	= 1000;
	static constexpr int32 DefaultPartitionSize = 2 << 20;
	static constexpr int32 MaxPartitionSize		= 32 << 20;
};

////////////////////////////////////////////////////////////////////////////////
struct FCountSize
{
	uint64 Count	= 0;
	uint64 Size		= 0;
};

////////////////////////////////////////////////////////////////////////////////
inline FCountSize operator+(const FCountSize& Lhs, const FCountSize& Rhs)
{
	return FCountSize
	{
		.Count = Lhs.Count + Rhs.Count,
		.Size = Lhs.Size + Rhs.Size
	};
}

////////////////////////////////////////////////////////////////////////////////
struct FContainerStats
{
	FCountSize	Chunks;
	FCountSize	Partitions;
	FCountSize	UploadedPartitions;
};

////////////////////////////////////////////////////////////////////////////////
struct FPartitionInfo
{
	FIoHash Hash = FIoHash::Zero;
	uint64	Size = 0;
};

////////////////////////////////////////////////////////////////////////////////
struct FContainerData
{
	FIoHash							UTocHash = FIoHash::Zero;
	FString							FilePath;
	FString							Name;
	TUniquePtr<FIoStoreReader>		Reader;
	FIoBuffer						Header;
	FIoBuffer						UToc;
	TArray<FIoStoreTocChunkInfo> 	ChunkInfos;
	TMap<FString, TArray<uint32>>	TagSets;
	TArray<FPartitionInfo>			UploadedPartitions;
	FContainerStats					Stats;
};

////////////////////////////////////////////////////////////////////////////////
using FUploadChunkCallback = TFunction<FIoStatus(FIoBuffer, const FIoHash&)>;

////////////////////////////////////////////////////////////////////////////////
struct FChunkValidationSettings
{
	bool	bValidate		= false;
	uint64	MaxRawChunkSize = 32 << 20;
};

////////////////////////////////////////////////////////////////////////////////
FIoStatus LoadContainers(
	TConstArrayView<FString> FilePaths,
	const FEncryptionKeys& EncryptionKeys,
	TPagedArray<FContainerData>& OutContainers,
	bool bIgnoreContainerFlags = false);

////////////////////////////////////////////////////////////////////////////////
FIoStatus LoadTagSets(
	const FString& FilePath,
	FTagSets& Out);

////////////////////////////////////////////////////////////////////////////////
FIoStatus CreateTagSets(
	const FTagSets& TagSets,
	FContainerData& Container);

////////////////////////////////////////////////////////////////////////////////
void GenerateTestChunks(
	FS3Client& Client,
	const FString& Bucket,
	const FString& BucketRelativePath,
	uint8 MaxExp = 16);

////////////////////////////////////////////////////////////////////////////////
FIoStatus DownloadPartitionChunkHashes(
	FS3Client& Client,
	const FString& Bucket,
	const FString& BucketRelativePath,
	int32 MaxTocListCount,
	int32 MaxTocDownloadCount,
	TSet<FIoHash>& Out);

////////////////////////////////////////////////////////////////////////////////
FIoStatus UploadPartitionChunkHashes(
	FS3Client& Client,
	const FString& Bucket,
	const FString& BucketRelativePath,
	const FString& TargetPlatform,
	const FString& BuildVersion,
	TConstArrayView<FPartitionInfo> Partitions);

////////////////////////////////////////////////////////////////////////////////
FIoStatus UploadContainer(
	FContainerData& Container,
	const FEncryptionKeys& EncryptionKeys,
	int32 MaxPartitionSize,
	UE::IoStore::Serialization::EOnDemandContainerEntryFlags ContainerFlags,
	const TSet<FIoHash>& ExistingChunks,
	FUploadChunkCallback&& Upload,
	UE::IoStore::Serialization::FOnDemandTocWriter& OutWriter,
	const FChunkValidationSettings& ValidationSettings = FChunkValidationSettings());

} // namespace UE::IoStore::Tool::Upload

