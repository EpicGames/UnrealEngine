// Copyright Epic Games, Inc. All Rights Reserved.

#include "UploadQueue.h"

#include <Async/Async.h>
#include <CoreGlobals.h>
#include <Misc/OutputDeviceRedirector.h>
#include <IO/IoStoreOnDemand.h>
#include "S3/S3Client.h"

namespace UE::IoStore::Tool
{


FUploadQueue::FUploadQueue(UE::FS3Client& InClient, const FString& InBucket, int32 ThreadCount)
	: Client(InClient)
	, Bucket(InBucket)
{
	ActiveThreadCount = ThreadCount;
	for (int32 Idx = 0; Idx < ThreadCount; ++Idx)
	{
		Threads.Add(AsyncThread([this]()
			{
				ThreadEntry();
			}));
	}
}

bool FUploadQueue::Enqueue(FStringView Key, FIoBuffer Payload)
{
	if (ActiveThreadCount == 0)
	{
		return false;
	}

	for (;;)
	{
		bool bEnqueued = false;
		{
			FScopeLock _(&CriticalSection);
			if (ConcurrentUploads < Threads.Num())
			{
				bEnqueued = Queue.Enqueue(FQueueEntry{ FString(Key), Payload });
			}
		}

		if (bEnqueued)
		{
			WakeUpEvent->Trigger();
			break;
		}

		UploadCompleteEvent->Wait();
	}

	return true;
}

void FUploadQueue::ThreadEntry()
{
	const int32 MaxAttempts = 3;
	const float RetryTimeouts[] = {0.5f, 1.0f, 2.0f};

	for (;;)
	{
		FQueueEntry Entry;
		bool bDequeued = false;
		{
			FScopeLock _(&CriticalSection);
			bDequeued = Queue.Dequeue(Entry);
		}

		if (!bDequeued)
		{
			if (bCompleteAdding)
			{
				break;
			}
			WakeUpEvent->Wait();
			continue;
		}

		ConcurrentUploads++;
		FS3PutObjectResponse Response;
		for (int32 Attempt = 0; Attempt < MaxAttempts; ++Attempt)
		{
			if (Response = Client.TryPutObject(FS3PutObjectRequest{ Bucket, Entry.Key, Entry.Payload.GetView() }); Response.IsOk())
			{
				break;
			}
			FPlatformProcess::Sleep(RetryTimeouts[Attempt]);
		}

		if (Response.IsOk())
		{
			UE_LOGF(LogIoStoreOnDemand, Log, "Uploaded chunk '%ls/%ls/%ls'", *Client.GetConfig().ServiceUrl, *Bucket, *Entry.Key);
		}
		else
		{
			TStringBuilder<256> ErrorResponse;
			Response.GetErrorResponse(ErrorResponse);

			UE_LOGF(LogIoStoreOnDemand, Warning, "Failed to upload chunk '%ls/%ls/%ls' (%ls)", *Client.GetConfig().ServiceUrl, *Bucket, *Entry.Key, ErrorResponse.ToString());
			ErrorCount++;
		}
		ConcurrentUploads--;
		UploadCompleteEvent->Trigger();
	}

	ActiveThreadCount--;
}

bool FUploadQueue::Flush()
{
	bCompleteAdding = true;
	for (int32 Idx = 0; Idx < Threads.Num(); ++Idx)
	{
		WakeUpEvent->Trigger();
	}

	for (TFuture<void>& Thread : Threads)
	{
		Thread.Wait();
	}

	return ErrorCount == 0;
}

} // namespace UE::IoStore::Tool
