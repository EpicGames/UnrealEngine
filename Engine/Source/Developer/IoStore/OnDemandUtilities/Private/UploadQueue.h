// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "IO/IoBuffer.h"

#include <Async/Future.h>
#include <Containers/Array.h>
#include <Containers/Queue.h>
#include <Containers/StringView.h>
#include <Containers/UnrealString.h>
#include <String/LexFromString.h>

namespace UE
{
	class FS3Client;
}

namespace UE::IoStore::Tool
{

////////////////////////////////////////////////////////////////////////////////
// s3 upload helper.... 
class FUploadQueue
{
public:
	FUploadQueue(UE::FS3Client& Client, const FString& Bucket, int32 ThreadCount);
	bool Enqueue(FStringView Key, FIoBuffer Payload);
	bool Flush();

private:
	void ThreadEntry();

	struct FQueueEntry
	{
		FString Key;
		FIoBuffer Payload;
	};

	UE::FS3Client& Client;
	TArray<TFuture<void>> Threads;
	FString Bucket;
	FCriticalSection CriticalSection;
	TQueue<FQueueEntry> Queue;
	FEventRef WakeUpEvent;
	FEventRef UploadCompleteEvent;
	std::atomic_int32_t ConcurrentUploads{ 0 };
	std::atomic_int32_t ActiveThreadCount{ 0 };
	std::atomic_int32_t ErrorCount{ 0 };
	std::atomic_bool bCompleteAdding{ false };
};

} // namespace UE::IoStore::Tool
